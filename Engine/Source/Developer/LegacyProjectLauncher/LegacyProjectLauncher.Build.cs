// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LegacyProjectLauncher : ModuleRules
{
	public LegacyProjectLauncher(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"LauncherServices",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DesktopPlatform",
				"ApplicationCore",
                "InputCore",
				"Slate",
				"SlateCore",
                
                "WorkspaceMenuStructure",
				"ToolWidgets",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"TargetDeviceServices",
			}
		);
	}
}
