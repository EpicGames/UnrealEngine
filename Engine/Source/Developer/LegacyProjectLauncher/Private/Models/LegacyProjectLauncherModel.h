// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "GameProjectHelper.h"
#include "ILauncher.h"
#include "ILauncherProfileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Templates/SharedPointer.h"


class FProjectLauncherModel;
class SLegacyProjectLauncher;


namespace ELauncherPanels
{
	/**
	 * Enumerates available launcher panels.
	 */
	enum Type
	{
		NoTask = 0,
		Launch,
		ProfileEditor,
		Progress,
		End,
	};
}


/**
 * Delegate type for launcher profile selection changes.
 *
 * The first parameter is the newly selected profile (or nullptr if none was selected).
 * The second parameter is the old selected profile (or nullptr if none was previous selected).
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectedSProjectLauncherProfileChanged, const ILauncherProfilePtr&, const ILauncherProfilePtr&)


/**
 * Implements the data model for the session launcher.
 */
class FProjectLauncherModel
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDeviceProxyManager The device proxy manager to use.
	 * @param InLauncher The launcher to use.
	 * @param InProfileManager The profile manager to use.
	 */
	FProjectLauncherModel(const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager, const TSharedRef<ILauncher>& InLauncher, const TSharedRef<ILauncherProfileManager>& InProfileManager)
		: DeviceProxyManager(InDeviceProxyManager)
		, SLegacyProjectLauncher(InLauncher)
		, ProfileManager(InProfileManager)
	{
		ProfileManager->OnProfileAdded().AddRaw(this, &FProjectLauncherModel::HandleProfileManagerProfileAdded);
		ProfileManager->OnProfileRemoved().AddRaw(this, &FProjectLauncherModel::HandleProfileManagerProfileRemoved);

		LoadConfig();
	}

	/** Destructor. */
	~FProjectLauncherModel()
	{
		ProfileManager->OnProfileAdded().RemoveAll(this);
		ProfileManager->OnProfileRemoved().RemoveAll(this);

		SaveConfig();
	}

public:

	/**
	 * Gets the model's device proxy manager.
	 *
	 * @return Device proxy manager.
	 */
	const TSharedRef<ITargetDeviceProxyManager>& GetDeviceProxyManager() const
	{
		return DeviceProxyManager;
	}

	/**
	 * Gets the model's launcher.
	 *
	 * @return SLegacyProjectLauncher.
	 */
	const TSharedRef<ILauncher>& GetSProjectLauncher() const
	{
		return SLegacyProjectLauncher;
	}

	/**
	 * Get the model's profile manager.
	 *
	 * @return Profile manager.
	 * @see GetProfiles
	 */
	const TSharedRef<ILauncherProfileManager>& GetProfileManager() const
	{
		return ProfileManager;
	}

	/**
	 * Gets the active profile.
	 *
	 * @return The active profile.
	 * @see GetProfiles, GetProfileManager, SelectProfile
	 */
	const TSharedPtr<ILauncherProfile>& GetSelectedProfile() const
	{
		return SelectedProfile;
	}

	/**
	 * Sets the active profile.
	 *
	 * @param InProfile The profile to assign as active,, or nullptr to deselect all.
	 * @see GetSelectedProfile
	 */
	void SelectProfile(const TSharedPtr<ILauncherProfile>& Profile)
	{
		if (!Profile.IsValid() || ProfileManager->GetAllProfiles().Contains(Profile))
		{
			if (Profile != SelectedProfile)
			{
				TSharedPtr<ILauncherProfile> PreviousProfile = SelectedProfile;
				SelectedProfile = Profile;

				ProfileSelectedDelegate.Broadcast(Profile, PreviousProfile);
			}
		}
	}

public:

	/**
	 * Returns a delegate to be invoked when the profile list has been modified.
	 *
	 * @return The delegate.
	 * @see OnProfileSelected
	 */
	FSimpleMulticastDelegate& OnProfileListChanged()
	{
		return ProfileListChangedDelegate;
	}

	/**
	 * Returns a delegate to be invoked when the selected profile changed.
	 *
	 * @return The delegate.
	 * @see OnProfileListChanged
	 */
	FOnSelectedSProjectLauncherProfileChanged& OnProfileSelected()
	{
		return ProfileSelectedDelegate;
	}

protected:

	/*
	 * Load all profiles from disk.
	 *
	 * @see SaveConfig
	 */
	void LoadConfig()
	{
		// restore the previous project selection
		FString ProjectPath;

		if (FPaths::IsProjectFilePathSet())
		{
			ProjectPath = FPaths::GetProjectFilePath();
		}
		else if (FGameProjectHelper::IsGameAvailable(FApp::GetProjectName()))
		{
			ProjectPath = FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
		}
		else if (GConfig != nullptr)
		{
			GConfig->GetString(TEXT("FProjectLauncherModel"), TEXT("SelectedProjectPath"), ProjectPath, GEngineIni);
		}

		ProfileManager->SetProjectPath(ProjectPath);
	}

	/*
	 * Saves all profiles to disk.

	 * @see LoadConfig
	 */
	void SaveConfig()
	{
		if (GConfig != nullptr && !FPaths::IsProjectFilePathSet() && !FGameProjectHelper::IsGameAvailable(FApp::GetProjectName()))
		{
			FString ProjectPath = ProfileManager->GetProjectPath();
			GConfig->SetString(TEXT("FProjectLauncherModel"), TEXT("SelectedProjectPath"), *ProjectPath, GEngineIni);
		}
	}

private:

	/** Callback for when a profile was added to the profile manager. */
	void HandleProfileManagerProfileAdded(const TSharedRef<ILauncherProfile>& Profile)
	{
		ProfileListChangedDelegate.Broadcast();

		SelectProfile(Profile);
	}

	/** Callback for when a profile was removed from the profile manager. */
	void HandleProfileManagerProfileRemoved(const TSharedRef<ILauncherProfile>& Profile)
	{
		ProfileListChangedDelegate.Broadcast();

		if (Profile == SelectedProfile)
		{
			const TArray<TSharedPtr<ILauncherProfile>>& Profiles = ProfileManager->GetAllProfiles();

			if (Profiles.Num() > 0)
			{
				SelectProfile(Profiles[0]);
			}
			else
			{
				SelectProfile(nullptr);
			}
		}
	}

private:

	/** Pointer to the device proxy manager. */
	TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager;

	/** Pointer to the launcher. */
	TSharedRef<ILauncher> SLegacyProjectLauncher;

	/** Pointer to the profile manager. */
	TSharedRef<ILauncherProfileManager> ProfileManager;

	/** Pointer to active profile. */
	TSharedPtr<ILauncherProfile> SelectedProfile;

private:

	/** A delegate to be invoked when the project path has been modified. */
	FSimpleMulticastDelegate ProjectPathChangedDelegate;

	/** A delegate to be invoked when the profile list has been modified. */
	FSimpleMulticastDelegate ProfileListChangedDelegate;

	/** A delegate to be invoked when the selected profile changed. */
	FOnSelectedSProjectLauncherProfileChanged ProfileSelectedDelegate;
};
