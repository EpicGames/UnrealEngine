// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "InputCoreTypes.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"


#define LOCTEXT_NAMESPACE "ProjectLauncherCommands"


/**
 * The project launcher UI commands
 */
class FProjectLauncherCommands
	: public TCommands<FProjectLauncherCommands>
{
public:

	/** Default constructor. */
	FProjectLauncherCommands()
		: TCommands<FProjectLauncherCommands>(
			"LauncherCommand",
			NSLOCTEXT("Contexts", "LauncherCommand", "Project Launcher Command"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{ }
	
public:

	//~ TCommands interface

	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(QuickLaunch, "Quick Launch", "Builds, cooks, and launches a build.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::L));
		UI_COMMAND(CreateBuild, "Build", "Creates a build.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::B));
		UI_COMMAND(DeployBuild, "Deploy Build", "Deploys a pre-made build.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::D));
		UI_COMMAND(AdvancedBuild, "Advanced...", "Advanced launcher.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::A));
		UI_COMMAND(CloseSettings, "Back", "Close the launch profile settings editor and go back to the launcher.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::None, EKeys::Escape));
		UI_COMMAND(RenameProfile, "Rename", "Rename the launch profile.", EUserInterfaceActionType::Button, FInputChord());// , FInputChord(EModifierKey::None, EKeys::F2));
		UI_COMMAND(DuplicateProfile, "Duplicate", "Duplicate the launch profile.", EUserInterfaceActionType::Button, FInputChord());//, FInputChord(EModifierKey::Control, EKeys::W));
		UI_COMMAND(DeleteProfile, "Delete", "Delete the launch profile.", EUserInterfaceActionType::Button, FInputChord());//, FInputChord(EModifierKey::Control, EKeys::Delete));
	}
	UE_ENABLE_OPTIMIZATION_SHIP

public:

	TSharedPtr<FUICommandInfo> CreateBuild;
	TSharedPtr<FUICommandInfo> DeployBuild;
	TSharedPtr<FUICommandInfo> QuickLaunch;
	TSharedPtr<FUICommandInfo> AdvancedBuild;
	TSharedPtr<FUICommandInfo> CloseSettings;
	TSharedPtr<FUICommandInfo> RenameProfile;
	TSharedPtr<FUICommandInfo> DuplicateProfile;
	TSharedPtr<FUICommandInfo> DeleteProfile;
};


#undef LOCTEXT_NAMESPACE
