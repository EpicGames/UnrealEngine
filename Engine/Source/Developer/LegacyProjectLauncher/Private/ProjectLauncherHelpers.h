// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

class FText;
struct FSlateBrush;


/* Static helpers
 *****************************************************************************/

// @todo gmp: move this into a shared library or create a SImageButton widget.
static TSharedRef<SButton> MakeImageButton(const FSlateBrush* ButtonImage, const FText& ButtonTip, const FOnClicked& OnClickedDelegate)
{
	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.OnClicked(OnClickedDelegate)
		.ToolTipText(ButtonTip)
		.ContentPadding(2)
		.VAlign(VAlign_Center)
		.ForegroundColor(FSlateColor::UseForeground())
		.Content()
		[
			SNew(SImage)
				.Image(ButtonImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
		];
}
