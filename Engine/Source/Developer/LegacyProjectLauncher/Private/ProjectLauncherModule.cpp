// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILauncherServicesModule.h"
#include "IProjectLauncherModule.h"
#include "ITargetDeviceServicesModule.h"
#include "Modules/ModuleManager.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "Styling/AppStyle.h"
#include "Framework/Docking/TabManager.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/SLegacyProjectLauncher.h"


static const FName ProjectLauncherTabName("LegacyProjectLauncher");


/**
 * Implements the SessionSProjectLauncher module.
 */
class FLegacyProjectLauncherModule
	: public ILegacyProjectLauncherModule
{
public:

	//~ IModuleInterface interface
	
	virtual void StartupModule() override
	{
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ProjectLauncherTabName, FOnSpawnTab::CreateRaw(this, &FLegacyProjectLauncherModule::SpawnProjectLauncherTab))
			.SetDisplayName(NSLOCTEXT("FProjectLauncherModule", "ProjectLauncherTabTitle", "Legacy Project Launcher (deprecated)"))
			.SetTooltipText(NSLOCTEXT("FProjectLauncherModule", "ProjectLauncherTooltipText", "Open the Legacy Project Launcher tab (note: this tool is deprecated)."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Launcher.TabIcon"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsPlatformsCategory());
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ProjectLauncherTabName);
	}

private:

	/**
	 * Creates a new session launcher tab.
	 *
	 * @param SpawnTabArgs The arguments for the tab to spawn.
	 * @return The spawned tab.
	 */
	TSharedRef<SDockTab> SpawnProjectLauncherTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);

		ILauncherServicesModule& ProjectLauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");
		ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");

		TSharedRef<FProjectLauncherModel> Model = MakeShareable(new FProjectLauncherModel(
			TargetDeviceServicesModule.GetDeviceProxyManager(),
			ProjectLauncherServicesModule.CreateLauncher(),
			ProjectLauncherServicesModule.GetProfileManager()
		));

		DockTab->SetContent(SNew(SLegacyProjectLauncher, DockTab, SpawnTabArgs.GetOwnerWindow(), Model));

		return DockTab;
	}
};


IMPLEMENT_MODULE(FLegacyProjectLauncherModule, LegacyProjectLauncher);
