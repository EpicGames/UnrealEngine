// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Archive/SProjectLauncherArchivePage.h"

#include "Styling/AppStyle.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherArchivePage"


/* SProjectLauncherCookPage structors
 *****************************************************************************/

SProjectLauncherArchivePage::~SProjectLauncherArchivePage()
{
}

/* SProjectLauncherCookPage interface
 *****************************************************************************/

void SProjectLauncherArchivePage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("ArchiveText", "Do you wish to archive?"))
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0, 0.0, 0.0, 0.0)
					[
						// archive mode check box
						SNew(SCheckBox)
							.IsChecked(this, &SProjectLauncherArchivePage::HandleArchiveIsChecked)
							.OnCheckStateChanged(this, &SProjectLauncherArchivePage::HandleArchiveCheckedStateChanged)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SBorder)
					.Padding(8.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Visibility(this, &SProjectLauncherArchivePage::HandleArchiveVisibility)
					[
						SNew(SVerticalBox)
					
						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
									.Text(LOCTEXT("ArchiveDirectoryTitle", "Archive Directory Path:"))
							]
					
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// archive path text box
										SNew(SEditableTextBox)
											.Text(this, &SProjectLauncherArchivePage::GetDirectoryPathText)
											.OnTextCommitted(this, &SProjectLauncherArchivePage::OnDirectoryTextCommitted)
									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.Padding(4.0, 0.0, 0.0, 0.0)
									[
										// browse button
										SNew(SButton)
											.ContentPadding(FMargin(6.0, 2.0))
											.IsEnabled(this, &SProjectLauncherArchivePage::IsDirectoryEditable)
											.Text(LOCTEXT("BrowseButtonText", "Browse..."))
											.ToolTipText(LOCTEXT("BrowseButtonToolTip", "Browse for the directory"))
											.OnClicked(this, &SProjectLauncherArchivePage::HandleBrowseButtonClicked)
									]
							]
					]
			]
	];
}


/* SProjectLauncherArchivePage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherArchivePage::HandleArchiveCheckedStateChanged(ECheckBoxState CheckState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetArchive(CheckState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherArchivePage::HandleArchiveIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsArchiving())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


EVisibility SProjectLauncherArchivePage::HandleArchiveVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsArchiving())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherArchivePage::GetDirectoryPathText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(SelectedProfile->GetArchiveDirectory());
	}

	return FText::GetEmpty();
}


FReply SProjectLauncherArchivePage::HandleBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(),
			GetDirectoryPathText().ToString(),
			FolderName
			);

		if (bFolderSelected)
		{
			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

			if (SelectedProfile.IsValid())
			{
				if (!FolderName.EndsWith(TEXT("/")))
				{
					FolderName += TEXT("/");
				}

				SelectedProfile->SetArchiveDirectory(FolderName);
			}
		}
	}

	return FReply::Handled();
}


bool SProjectLauncherArchivePage::IsDirectoryEditable() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->IsArchiving();
	}
	return false;
}


void SProjectLauncherArchivePage::OnDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	//if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetArchiveDirectory(InText.ToString());
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
