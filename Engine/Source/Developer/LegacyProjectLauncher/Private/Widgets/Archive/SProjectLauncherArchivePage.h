// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

enum class ECheckBoxState : uint8;

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherArchivePage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherArchivePage) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SProjectLauncherArchivePage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	/** Callback for changing the checked state of an archive option. */
	void HandleArchiveCheckedStateChanged( ECheckBoxState CheckState );

	/** Callback for determining whether an archive option is checked. */
	ECheckBoxState HandleArchiveIsChecked() const;

	/** Callback for determining if the archive options should be displayed. */
	EVisibility HandleArchiveVisibility( ) const;

	/** Get archive directory text */
	FText GetDirectoryPathText() const;
		
	/** Handle browse button for archive directory*/
	FReply HandleBrowseButtonClicked();

	/** Whether archive directory is editable */
	bool IsDirectoryEditable() const;

	/** Handle commit event for archive directory text */
	void OnDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

private:

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;
};
