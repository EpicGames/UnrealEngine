// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Build/SProjectLauncherBuildPage.h"

#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboBox.h"

#include "Widgets/Cook/SProjectLauncherCookedPlatforms.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Shared/SProjectLauncherBuildConfigurationSelector.h"
#include "Widgets/Shared/SProjectLauncherFormLabel.h"
#include "Widgets/Shared/SProjectLauncherBuildTargetSelector.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherBuildPage"


/* SProjectLauncherCookPage structors
 *****************************************************************************/

SProjectLauncherBuildPage::~SProjectLauncherBuildPage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherCookPage interface
 *****************************************************************************/

void SProjectLauncherBuildPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	// create cook modes menu
	FMenuBuilder BuildModeMenuBuilder(true, NULL);
	{
		FUIAction AutoAction(FExecuteAction::CreateSP(this, &SProjectLauncherBuildPage::HandleBuildModeMenuEntryClicked, ELauncherProfileBuildModes::Auto));
		BuildModeMenuBuilder.AddMenuEntry(LOCTEXT("BuildMode_AutoAction", "Detect Automatically"), LOCTEXT("BuildMode_AutoActionHint", "Detect whether the project needs to be built automatically."), FSlateIcon(), AutoAction);

		FUIAction BuildAction(FExecuteAction::CreateSP(this, &SProjectLauncherBuildPage::HandleBuildModeMenuEntryClicked, ELauncherProfileBuildModes::Build));
		BuildModeMenuBuilder.AddMenuEntry(LOCTEXT("BuildMode_BuildAction", "Build"), LOCTEXT("BuildMode_BuildActionHint", "Build the target."), FSlateIcon(), BuildAction);

		FUIAction DoNotBuildAction(FExecuteAction::CreateSP(this, &SProjectLauncherBuildPage::HandleBuildModeMenuEntryClicked, ELauncherProfileBuildModes::DoNotBuild));
		BuildModeMenuBuilder.AddMenuEntry(LOCTEXT("BuildMode_DoNotBuildAction", "Do Not Build"), LOCTEXT("DoNotCookActionHint", "Do not build the target."), FSlateIcon(), DoNotBuildAction);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("BuildText", "Do you wish to build?"))
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0, 0.0, 0.0, 0.0)
					[
						// cooking mode menu
						SNew(SComboButton)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(this, &SProjectLauncherBuildPage::HandleBuildModeComboButtonContentText)
						]
						.ContentPadding(FMargin(6.0, 2.0))
						.MenuContent()
						[
							BuildModeMenuBuilder.MakeWidget()
						]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 3, 0, 3)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Visibility(this, &SProjectLauncherBuildPage::ShowBuildConfiguration)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SProjectLauncherFormLabel)
									.ErrorToolTipText(NSLOCTEXT("SProjectLauncherBuildValidation", "NoBuildConfigurationSelectedError", "A Build Configuration must be selected."))
									.ErrorVisibility(this, &SProjectLauncherBuildPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoBuildConfigurationSelected)
									.LabelText(LOCTEXT("ConfigurationComboBoxLabel", "Build Configuration:"))
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								// build configuration selector
								SNew(SProjectLauncherBuildConfigurationSelector)
									.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
									.OnConfigurationSelected(this, &SProjectLauncherBuildPage::HandleBuildConfigurationSelectorConfigurationSelected)
									.Text(this, &SProjectLauncherBuildPage::HandleBuildConfigurationSelectorText)
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SProjectLauncherBuildTargetSelector, Model.ToSharedRef())
									.UseProfile(true)
							]

					]
			]

        + SVerticalBox::Slot()
            .AutoHeight()
            .Padding(0.0f, 8.0f, 0.0f, 0.0f)
            [
                SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						SNew(SVerticalBox)

	/*                    + SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SButton)
							.Text(LOCTEXT("GenDSYMText", "Generate DSYM"))
							.IsEnabled(this, &SProjectLauncherBuildPage::HandleGenDSYMButtonEnabled)
							.OnClicked(this, &SProjectLauncherBuildPage::HandleGenDSYMClicked)
						]*/
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							// build mode check box
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherBuildPage::HandleUATIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherBuildPage::HandleUATCheckedStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UATCheckBoxTooltip", "If checked, UAT will be built as part of the build."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("UATCheckBoxText", "Build UAT"))
								]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 8.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherFormLabel)
							.LabelText(LOCTEXT("CommandLineTextBoxLabel", "Additional Command Line Parameters:"))
							.ToolTipText(LOCTEXT("CommandLineTextBoxToolTip", "Parameters specified here will be passed to the app on launch"))
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// Additional launch parameters
							SAssignNew(CommandLineTextBox, SEditableTextBox)
							.Text(this, &SProjectLauncherBuildPage::GetCommandLineText)
							.OnTextChanged(this, &SProjectLauncherBuildPage::HandleCommandLineTextBoxChanged)
						]
				   ]
            ]

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SProjectLauncherCookedPlatforms, InModel)
					.Visibility(this, &SProjectLauncherBuildPage::HandleBuildPlatformVisibility)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherBuildPage::HandleProfileManagerProfileSelected);
}


/* SProjectLauncherBuildPage implementation
 *****************************************************************************/

bool SProjectLauncherBuildPage::GenerateDSYMForProject(const FString& ProjectName, const FString& Configuration)
{
    // UAT executable
    FString ExecutablePath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + FString(TEXT("Build")) / TEXT("BatchFiles"));
#if PLATFORM_MAC
    FString Executable = TEXT("RunUAT.command");
#elif PLATFORM_LINUX
    FString Executable = TEXT("RunUAT.sh");
#else
    FString Executable = TEXT("RunUAT.bat");
#endif

    // build UAT command line parameters
    FString CommandLine;
    CommandLine = FString::Printf(TEXT("GenerateDSYM -project=%s -config=%s"),
        *ProjectName,
        *Configuration);

    // launch the builder and monitor its process
    FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*(ExecutablePath / Executable), *CommandLine, false, false, false, NULL, 0, *ExecutablePath, NULL);
	if (ProcessHandle.IsValid())
	{
		FPlatformProcess::CloseProc(ProcessHandle);
		return true;
	}

	return false;
}


/* SProjectLauncherBuildPage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FText SProjectLauncherBuildPage::HandleBuildModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfileBuildModes::Type BuildMode = SelectedProfile->GetBuildMode();

		if (BuildMode == ELauncherProfileBuildModes::Auto)
		{
			return LOCTEXT("BuildModeComboButton_Auto", "Detect Automatically");
		}

		if (BuildMode == ELauncherProfileBuildModes::Build)
		{
			return LOCTEXT("BuildModeComboButton_Build", "Build");
		}

		if (BuildMode == ELauncherProfileBuildModes::DoNotBuild)
		{
			return LOCTEXT("BuildModeComboButton_DoNotBuild", "Do not build");
		}

		return LOCTEXT("BuildModeComboButtonDefaultText", "Select...");
	}

	return FText();
}


void SProjectLauncherBuildPage::HandleBuildModeMenuEntryClicked(ELauncherProfileBuildModes::Type BuildMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetBuildMode(BuildMode);
	}
}

void SProjectLauncherBuildPage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	// reload settings
}


EVisibility SProjectLauncherBuildPage::HandleBuildPlatformVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::DoNotCook && SelectedProfile->GetDeploymentMode() == ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FReply SProjectLauncherBuildPage::HandleGenDSYMClicked()
{
    ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

    if(SelectedProfile.IsValid())
    {
        if (!SelectedProfile->HasValidationError(ELauncherProfileValidationErrors::NoProjectSelected))
        {
            FString ProjectName = SelectedProfile->GetProjectName();
            EBuildConfiguration ProjectConfig = SelectedProfile->GetBuildConfiguration();

            GenerateDSYMForProject(ProjectName, LexToString(ProjectConfig));
        }
    }

    return FReply::Handled();
}


bool SProjectLauncherBuildPage::HandleGenDSYMButtonEnabled() const
{
    ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

    if (SelectedProfile.IsValid())
    {
        if (!SelectedProfile->HasValidationError(ELauncherProfileValidationErrors::NoProjectSelected))
        {
            return true;
        }
    }

    return false;
}


EVisibility SProjectLauncherBuildPage::ShowBuildConfiguration() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}


void SProjectLauncherBuildPage::HandleBuildConfigurationSelectorConfigurationSelected(EBuildConfiguration Configuration)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetBuildConfiguration(Configuration);
	}
}


FText SProjectLauncherBuildPage::HandleBuildConfigurationSelectorText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(LexToString(SelectedProfile->GetBuildConfiguration()));
	}

	return FText::GetEmpty();
}


EVisibility SProjectLauncherBuildPage::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

void SProjectLauncherBuildPage::HandleUATCheckedStateChanged(ECheckBoxState CheckState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetBuildUAT(CheckState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherBuildPage::HandleUATIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsBuildingUAT())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

void SProjectLauncherBuildPage::HandleCommandLineTextBoxChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetAdditionalCommandLineParameters(InText.ToString());
	}
}

FText SProjectLauncherBuildPage::GetCommandLineText() const
{
	FString CommandLine;
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		CommandLine = SelectedProfile->GetAdditionalCommandLineParameters();
	}

	return FText::FromString(CommandLine);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
