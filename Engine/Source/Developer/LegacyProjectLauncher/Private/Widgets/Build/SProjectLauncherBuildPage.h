// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "ILauncherProfile.h"

class Error;
enum class ECheckBoxState : uint8;

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherBuildPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherBuildPage) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SProjectLauncherBuildPage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

protected:

	/**
	 * Generates the DSYM for the project.
	 *
	 * @param ProjectName The name of the project.
	 * @param Configuration The build configuration name.
	 * @return true on success, false otherwise.
	 */
    bool GenerateDSYMForProject( const FString& ProjectName, const FString& Configuration );

private:

	// Callback for getting the content text of the 'Build Mode' combo button.
	FText HandleBuildModeComboButtonContentText() const;

	// Callback for clicking an item in the 'Build Mode' menu.
	void HandleBuildModeMenuEntryClicked(ELauncherProfileBuildModes::Type BuildMode);

	/** Callback for changing the selected profile in the profile manager. */
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

	/** Callback for determining if the build platform list should be displayed. */
	EVisibility HandleBuildPlatformVisibility( ) const;

    /** Callback for pressing the Advanced Setting - Generate DSYM button. */
    FReply HandleGenDSYMClicked();

	/** Callback for getting the enabled state of the Generate DSYM button. */
    bool HandleGenDSYMButtonEnabled() const;

	/** Callback for determining if the build configuration should be shown */
	EVisibility ShowBuildConfiguration() const;

	/** Callback for selecting a build configuration. */
	void HandleBuildConfigurationSelectorConfigurationSelected(EBuildConfiguration Configuration);

	/** Callback for getting the content text of the build configuration selector. */
	FText HandleBuildConfigurationSelectorText() const;

	/** Callback for determining the visibility of a validation error icon. */
	EVisibility HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const;

	/** Callback for changing the checked state of a platform menu check box. */
	void HandleUATCheckedStateChanged( ECheckBoxState CheckState );

	/** Callback for determining whether a platform menu entry is checked. */
	ECheckBoxState HandleUATIsChecked() const;

	/** Callback for retrieving the current command line params for the profile */
	FText GetCommandLineText() const;

	/** Callback for changing the additional command line params */
	void HandleCommandLineTextBoxChanged(const FText& InText);

private:

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	TSharedPtr<class SEditableTextBox> CommandLineTextBox;
};
