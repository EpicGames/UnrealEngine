// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Build/SProjectLauncherBuildTaskSettings.h"

#include "Fonts/SlateFontInfo.h"
#include "Styling/CoreStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Build/SProjectLauncherBuildPage.h"
#include "Widgets/Project/SProjectLauncherProjectPage.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherBuildTaskSettings"


/* SProjectLauncherBuildTaskSettings structors
 *****************************************************************************/

SProjectLauncherBuildTaskSettings::~SProjectLauncherBuildTaskSettings()
{

}


/* SProjectLauncherBuildTaskSettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherBuildTaskSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SScrollBox)

				+ SScrollBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SGridPanel)
							.FillColumn(1, 1.0f)

						// project section
						+ SGridPanel::Slot(0, 0)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Top)
							[
								SNew(STextBlock)
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
									.Text(LOCTEXT("ProjectSectionHeader", "Project"))
							]

						+ SGridPanel::Slot(1, 0)
							.Padding(32.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SProjectLauncherProjectPage, InModel)
							]

						// build section
						+ SGridPanel::Slot(0, 7)
							.ColumnSpan(3)
							.Padding(0.0f, 16.0f)
							[
								SNew(SSeparator)
									.Orientation(Orient_Horizontal)
							]

						+ SGridPanel::Slot(0, 8)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Top)
							[
								SNew(STextBlock)
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
									.Text(LOCTEXT("DeploySectionHeader", "Deploy"))
							]

						+ SGridPanel::Slot(1, 8)
							.Padding(32.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SProjectLauncherBuildPage, InModel)
							]
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherBuildTaskSettings callbacks
*****************************************************************************/


#undef LOCTEXT_NAMESPACE
