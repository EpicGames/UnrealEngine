// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Cook/SProjectLauncherCookByTheBookSettings.h"

#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Shared/SProjectLauncherBuildConfigurationSelector.h"
#include "Widgets/Shared/SProjectLauncherFormLabel.h"
#include "Widgets/Cook/SProjectLauncherMapListRow.h"
#include "Widgets/Cook/SProjectLauncherCultureListRow.h"
#include "Widgets/Cook/SProjectLauncherCookedPlatforms.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SExpandableArea.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherCookByTheBookSettings"


/* SProjectLauncherCookByTheBookSettings structors
 *****************************************************************************/

SProjectLauncherCookByTheBookSettings::~SProjectLauncherCookByTheBookSettings()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherCookByTheBookSettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherCookByTheBookSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowSimple)
{
	Model = InModel;

	ChildSlot
	[
		InShowSimple ? MakeSimpleWidget() : MakeComplexWidget()
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherCookByTheBookSettings::HandleProfileManagerProfileSelected);

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->OnProjectChanged().AddSP(this, &SProjectLauncherCookByTheBookSettings::HandleProfileProjectChanged);
	}

	ShowMapsChoice = EShowMapsChoices::ShowAllMaps;

	RefreshMapList();
	RefreshCultureList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherCookByTheBookSettings implementation
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SProjectLauncherCookByTheBookSettings::MakeComplexWidget()
{
	TSharedRef<SWidget> Widget = SNew(SVerticalBox)

	+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(256.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(8.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SProjectLauncherFormLabel)
							.ErrorToolTipText(NSLOCTEXT("ProjectLauncherBuildValidation", "NoCookedPlatformSelectedError", "At least one Platform must be selected when cooking by the book."))
							.ErrorVisibility(this, &SProjectLauncherCookByTheBookSettings::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoPlatformSelected)
							.LabelText(LOCTEXT("CookedPlatformsLabel", "Cooked Platforms:"))
					]

					+ SVerticalBox::Slot()
						.FillHeight(1.0)
						.Padding(0.0f, 2.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherCookedPlatforms, Model.ToSharedRef())
						]
				]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(256.0f)
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SProjectLauncherFormLabel)
							.ErrorToolTipText(NSLOCTEXT("ProjectLauncherBuildValidation", "NoCookedCulturesSelectedError", "At least one Culture must be selected when cooking by the book."))
							.ErrorVisibility(this, &SProjectLauncherCookByTheBookSettings::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoCookedCulturesSelected)
							.LabelText(LOCTEXT("CookedCulturesLabel", "Cooked Cultures:"))
					]

				+ SVerticalBox::Slot()
					.FillHeight(1.0)
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					[
						// culture menu
						SAssignNew(CultureListView, SListView<TSharedPtr<FString> >)
							.HeaderRow
							(
								SNew(SHeaderRow)
									.Visibility(EVisibility::Collapsed)

								+ SHeaderRow::Column("Culture")
									.DefaultLabel(LOCTEXT("CultureListMapNameColumnHeader", "Culture"))
									.FillWidth(1.0f)
							)
							.ListItemsSource(&CultureList)
							.OnGenerateRow(this, &SProjectLauncherCookByTheBookSettings::HandleCultureListViewGenerateRow)
							.SelectionMode(ESelectionMode::None)
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 6.0f, 0.0f, 4.0f)
					[
						SNew(SSeparator)
							.Orientation(Orient_Horizontal)
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.HAlign(HAlign_Right)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SelectLabel", "Select:"))
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(8.0f, 0.0f)
							[
								// all cultures hyper link
								SNew(SHyperlink)
									.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkNavigate, true)
									.Text(LOCTEXT("AllPlatformsHyperlinkLabel", "All"))
									.ToolTipText(LOCTEXT("AllPlatformsButtonTooltip", "Select all available platforms."))
									.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkVisibility)
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								// no cultures hyper link
								SNew(SHyperlink)
									.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkNavigate, false)
									.Text(LOCTEXT("NoCulturesHyperlinkLabel", "None"))
									.ToolTipText(LOCTEXT("NoCulturesHyperlinkTooltip", "Deselect all platforms."))
									.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkVisibility)
							]
					]
			]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(256.0f)
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(8.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookedMapsLabel", "Cooked Maps:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								// all maps radio button
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowAllMaps)
									.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowAllMaps)
									.Style(FAppStyle::Get(), "RadioButton")
									[
										SNew(STextBlock)
											.Text(LOCTEXT("AllMapsCheckBoxText", "Show all"))
									]
							]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.0f, 0.0f, 0.0f, 0.0f)
								[
									// cooked maps radio button
									SNew(SCheckBox)
										.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowCookedMaps)
										.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowCookedMaps)
										.Style(FAppStyle::Get(), "RadioButton")
										[
											SNew(STextBlock)
												.Text(LOCTEXT("CookedMapsCheckBoxText", "Show cooked"))
										]
								]

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(8.0f, 0.0f, 0.0f, 0.0f)
								[
									// missing maps radio button
									SNew(SCheckBox)
										.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowMissingMaps)
										.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowMissingMaps)
										.Style(FAppStyle::Get(), "RadioButton")
										[
											SNew(STextBlock)
												.Text(LOCTEXT("MissingMapsCheckBoxText", "Show missing"))
										]
								]
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// map list
							SAssignNew(MapListView, SListView<TSharedPtr<FString> >)
								.HeaderRow
								(
									SNew(SHeaderRow)
										.Visibility(EVisibility::Collapsed)

									+ SHeaderRow::Column("MapName")
										.DefaultLabel(LOCTEXT("MapListMapNameColumnHeader", "Map"))
										.FillWidth(1.0f)
								)
								.ListItemsSource(&MapList)
								.OnGenerateRow(this, &SProjectLauncherCookByTheBookSettings::HandleMapListViewGenerateRow)
								.SelectionMode(ESelectionMode::None)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
								.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleNoMapSelectedBoxVisibility)

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush(TEXT("Icons.Warning")))
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(4.0f, 0.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.Text(this, &SProjectLauncherCookByTheBookSettings::HandleNoMapsTextBlockText)
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 6.0f, 0.0f, 4.0f)
						[
							SNew(SSeparator)
								.Orientation(Orient_Horizontal)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Right)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("SelectLabel", "Select:"))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.0f, 0.0f)
								[
									// all maps hyper link
									SNew(SHyperlink)
										.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllMapsHyperlinkNavigate, true)
										.Text(LOCTEXT("AllMapsHyperlinkLabel", "All"))
										.ToolTipText(LOCTEXT("AllMapsHyperlinkTooltip", "Select all available maps."))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									// no maps hyper link
									SNew(SHyperlink)
										.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllMapsHyperlinkNavigate, false)
										.Text(LOCTEXT("NoMapsHyperlinkLabel", "None"))
										.ToolTipText(LOCTEXT("NoMapsHyperlinkTooltip", "Deselect all maps."))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]
						]
				]
		]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("PatchingAreaTitle", "Release / DLC / Patching Settings"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						SNew(SVerticalBox)

						//////////////////////////////////////////////////////////////////////////
						// create release version options
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								// CreateReleaseVersion check box
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("CreateReleaseVersionCheckBoxTooltip", "Create a release version of the game for distribution."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("CreateReleaseVersionBoxText", "Create a release version of the game for distribution."))
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SProjectLauncherFormLabel)
									.LabelText(LOCTEXT("CreateReleaseVersionTextBoxLabel", "Name of the new release to create."))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.Padding(0.0, 0.0, 0.0, 3.0)
								[
									// CreateReleaseVersion textbox
									SNew(SEditableTextBox)
										.ToolTipText(LOCTEXT("CreateReleaseVersionTextBoxTooltip", "Name of the new release to create."))
										.Text(this, &SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionNameTextBlockText)
										.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionNameCommitted)
								]
							]

						+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SProjectLauncherFormLabel)
									.ToolTipText(LOCTEXT("BasedOnReleaseVersionTextBoxToolTip", "The release version which this DLC / Patch / Next release is based on."))
									.LabelText(LOCTEXT("BasedOnReleaseVersionTextBoxLabel", "Release version this is based on."))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// BasedOnReleaseVersion textbox
										SNew(SEditableTextBox)
											.ToolTipText(LOCTEXT("NextReleaseVersionTextBoxTooltip", "Release version to base the next release / DLC / patch on."))
											.Text(this, &SProjectLauncherCookByTheBookSettings::HandleBasedOnReleaseVersionNameTextBlockText)
											.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleBasedOnReleaseVersionNameCommitted)
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SProjectLauncherFormLabel)
									.ToolTipText(LOCTEXT("OriginalReleaseVersionTextBoxToolTip", "The original release version which this DLC / Patch / Next release is based on."))
									.LabelText(LOCTEXT("OriginalReleaseVersionTextBoxLabel", "Original release version of this application."))
									.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionVisibility)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// original release version text box
										SNew(SEditableTextBox)
											.ToolTipText(LOCTEXT("OriginalReleaseVersionTextBoxTooltip", "The original release version"))
											.Text(this, &SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionNameTextBlockText)
											.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionNameCommitted)
											.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionVisibility)
									]
							]

						// end create release version
						//////////////////////////////////////////////////////////////////////////

						// generate patch params
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								// Generate patch checkbox
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleGeneratePatchCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleGeneratePatchCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("GeneratePatchCheckBoxTooltip", "If checked, content will be diffed against source content and only changed files will be included in new pak"))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("GeneratePatchCheckBoxText", "Generate patch"))
									]
							]

						// end generate patch options
						//////////////////////////////////////////////////////////////////////////

						// generate dlc options
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								// BuildDLC checkbox
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleBuildDLCCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleBuildDLCCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("BuildDLCCheckBoxTooltip", "If checked, DLC will be built without the content released with the original game."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("BuildDLCCheckBoxText", "Build DLC"))
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SProjectLauncherFormLabel)
									.LabelText(LOCTEXT("DLCNameTextBoxLabel", "Name of the DLC to build."))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// repository path text box
										SNew(SEditableTextBox)
											.ToolTipText(LOCTEXT("DLCNameTextBoxTooltip", "Name of DLC to build."))
											.Text(this, &SProjectLauncherCookByTheBookSettings::HandleDLCNameTextBlockText)
											.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleDLCNameCommitted)
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								// Include engine content checkbox
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleDLCIncludeEngineContentCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleDLCIncludeEngineContentCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("HandleDLCIncludeEngineContentCheckBoxTooltip", "If checked, DLC will include engine content which was not included in original release, if not checked will error when accessing content from engine directory."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("HandleDLCIncludeEngineContentCheckBoxText", "Include engine content"))
									]
							]

						// end generate dlc 
						//////////////////////////////////////////////////////////////////////////
				]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
				.InitiallyCollapsed(true)
				.Padding(8.0f)
				.BodyContent()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							// incremental cook checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("IncrementalCheckBoxTooltip", "If checked, only modified content will be cooked, resulting in much faster cooking times. It is recommended to enable this option whenever possible."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("IncrementalCheckBoxText", "Iterative cooking: Only cook content modified from previous cook"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// Compress content checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleCompressedCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleCompressedCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("CompressedCheckboxToolTip", "If checked, content will be generated compressed.  These will be smaller but potentially take longer to load"))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("CompressedCheckBoxText", "Compress content"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// generate new patch level checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleAddPatchLevelCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleAddPatchLevelCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("AddPatchLevelCheckBoxTooltip", "If checked, a new numbered pak will be generated with patch content"))
								.Content()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("AddPatchLevelCheckBoxText", "Add a new patch tier"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// Unversioned cooked content checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UnversionedCheckBoxTooltip", "If checked, the version is assumed to be current at load. This is potentially dangerous, but results in smaller patch sizes."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("UnversionedCheckBoxText", "Save packages without versions"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// Create .pak files checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UnrealPakCheckBoxTooltip", "If checked, the content will be deployed as a single UnrealPak file instead of many separate files."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("UnrealPakCheckBoxText", "Store all content in a single file (UnrealPak)"))
								]
						]

					+ SVerticalBox::Slot()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						.AutoHeight()
						[
							// Encrypt Ini Files checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleEncryptIniFilesCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleEncryptIniFilesCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("EncryptIniFilesCheckboxToolTip", "If checked, ini files stored inside pak file will be encrypted."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("EncryptIniFilesCheckBoxText", "Encrypt ini files (only with use pak file)"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// generate chunks checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleGenerateChunksCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleGenerateChunksCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("GenerateChunksCheckBoxTooltip", "If checked, the content will be deployed as multiple UnrealPak files instead of many separate files."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("GenerateChunksCheckBoxText", "Generate Chunks"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// don't include editor content checkbox
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleDontIncludeEditorContentCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleDontIncludeEditorContentCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("DontIncludeEditorContentCheckBoxTooltip", "If checked the cooker will skip editor content and not include it in the build."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("DontIncludeEditorContentCheckBoxText", "Don't Include editor content in the build"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(SExpandableArea)
								.AreaTitle(LOCTEXT("HttpChunkInstallSettingsAreaTitle", "Http Chunk Install Settings"))
								.InitiallyCollapsed(true)
								.Padding(FMargin(4.0f, 0.0f))
								.BodyContent()
								[
									SNew(SVerticalBox)

									+ SVerticalBox::Slot()
										.AutoHeight()
										.Padding(0.0f, 4.0f, 0.0f, 0.0f)
										[
											// HTTPChunkInstalls checkbox
											SNew(SCheckBox)
												.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallCheckBoxIsChecked)
												.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallCheckBoxCheckStateChanged)
												.Padding(FMargin(4.0f, 0.0f))
												.ToolTipText(LOCTEXT("HttpChunkInstallCheckBoxTooltip", "If checked, the content will be split into multiple paks and stored as data that can be downloaded."))
												.Content()
												[
													SNew(STextBlock)
														.Text(LOCTEXT("HttpChunkInstallCheckBoxText", "Create Http Chunk Install data"))
												]
										]

									+ SVerticalBox::Slot()
										.AutoHeight()
										[
											SNew(STextBlock)
												.Text(LOCTEXT("HttpChunkInstallDataPathLabel", "Http Chunk Install Data Path:"))
										]

									+ SVerticalBox::Slot()
										.AutoHeight()
										.Padding(0.0, 4.0, 0.0, 0.0)
										[
											SNew(SHorizontalBox)

											+ SHorizontalBox::Slot()
												.FillWidth(1.0)
												.Padding(0.0f, 4.0f, 0.0f, 0.0f)
												[
													// repository path text box
													SAssignNew(HttpChunkInstallDirectoryTextBox, SEditableTextBox)
														.Text(this, &SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryText)
														.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryTextCommitted)
														.OnTextChanged(this, &SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryTextChanged)
												]

											+ SHorizontalBox::Slot()
												.AutoWidth()
												.HAlign(HAlign_Right)
												.Padding(4.0, 0.0, 0.0, 0.0)
												[
													// browse button
													SNew(SButton)
														.ContentPadding(FMargin(6.0, 2.0))
														.IsEnabled(true)
														.Text(LOCTEXT("BrowseButtonText", "Browse..."))
														.ToolTipText(LOCTEXT("BrowseButtonToolTip", "Browse for the Http Chunk Install Data directory"))
														.OnClicked(this, &SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallBrowseButtonClicked)
												]
										]
				
									+ SVerticalBox::Slot()
										.AutoHeight()
										.Padding(0.0f, 4.0f, 0.0f, 0.0f)
										[
											SNew(SProjectLauncherFormLabel)
												.LabelText(LOCTEXT("HttpChunkInstallReleaseTextBoxLabel", "Http Chunk Install Release Name:"))
										]

									+ SVerticalBox::Slot()
										.AutoHeight()
										.Padding(0.0f, 4.0f, 0.0f, 0.0f)
										[
											// cooker command line options
											SNew(SEditableTextBox)
												.ToolTipText(LOCTEXT("HttpChunkInstallReleaseTextBoxTooltip", "Name of this version of the Http Chunk Install data."))
												.Text(this, &SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallNameTextBlockText)
												.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallNameCommitted)
										]
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 12.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookConfigurationSelectorLabel", "Cooker build configuration:"))							
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// cooker build configuration selector
							SNew(SProjectLauncherBuildConfigurationSelector)
								.OnConfigurationSelected(this, &SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorConfigurationSelected)
								.Text(this, &SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorText)
								.ToolTipText(LOCTEXT("CookConfigurationToolTipText", "Sets the build configuration to use for the cooker commandlet."))
						]

					// additional cooker options text box
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 8.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookerOptionsTextBoxLabel", "Additional Cooker Options:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// cooker command line options
							SNew(SEditableTextBox)
								.ToolTipText(LOCTEXT("CookerOptionsTextBoxTooltip", "Additional cooker command line parameters can be specified here."))
								.Text(this, &SProjectLauncherCookByTheBookSettings::HandleCookOptionsTextBlockText)
								.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleCookerOptionsCommitted)
						]
				]
		];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SProjectLauncherCookByTheBookSettings::MakeSimpleWidget()
{
	TSharedRef<SWidget> Widget = SNew(SVerticalBox)

	+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(256.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(8.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
						[
							SNew(SProjectLauncherFormLabel)
								.ErrorToolTipText(NSLOCTEXT("ProjectLauncherBuildValidation", "NoCookedPlatformSelectedError", "At least one Platform must be selected when cooking by the book."))
								.ErrorVisibility(this, &SProjectLauncherCookByTheBookSettings::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoPlatformSelected)
								.LabelText(LOCTEXT("CookedPlatformsLabel", "Cooked Platforms:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 2.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherCookedPlatforms, Model.ToSharedRef())
						]
				]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(256.0f)
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(8.0f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookedMapsLabel", "Cooked Maps:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									// all maps radio button
									SNew(SCheckBox)
										.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowAllMaps)
										.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowAllMaps)
										.Style(FAppStyle::Get(), "RadioButton")
										[
											SNew(STextBlock)
												.Text(LOCTEXT("AllMapsCheckBoxText", "Show all"))
										]
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.0f, 0.0f, 0.0f, 0.0f)
								[
									// cooked maps radio button
									SNew(SCheckBox)
										.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowCookedMaps)
										.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowCookedMaps)
										.Style(FAppStyle::Get(), "RadioButton")
										[
											SNew(STextBlock)
												.Text(LOCTEXT("CookedMapsCheckBoxText", "Show cooked"))
										]
								]

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(8.0f, 0.0f, 0.0f, 0.0f)
								[
									// missing maps radio button
									SNew(SCheckBox)
										.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked, EShowMapsChoices::ShowMissingMaps)
										.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged, EShowMapsChoices::ShowMissingMaps)
										.Style(FAppStyle::Get(), "RadioButton")
										[
											SNew(STextBlock)
												.Text(LOCTEXT("MissingMapsCheckBoxText", "Show missing"))
										]
								]
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// map list
							SAssignNew(MapListView, SListView<TSharedPtr<FString> >)
								.HeaderRow
								(
									SNew(SHeaderRow)
										.Visibility(EVisibility::Collapsed)

									+ SHeaderRow::Column("MapName")
										.DefaultLabel(LOCTEXT("MapListMapNameColumnHeader", "Map"))
										.FillWidth(1.0f)
								)
								.ListItemsSource(&MapList)
								.OnGenerateRow(this, &SProjectLauncherCookByTheBookSettings::HandleMapListViewGenerateRow)
								.SelectionMode(ESelectionMode::None)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
								.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleNoMapSelectedBoxVisibility)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SImage)
									.Image(FAppStyle::GetBrush(TEXT("Icons.Warning")))
							]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(4.0f, 0.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.Text(this, &SProjectLauncherCookByTheBookSettings::HandleNoMapsTextBlockText)
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 6.0f, 0.0f, 4.0f)
						[
							SNew(SSeparator)
								.Orientation(Orient_Horizontal)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Right)
								[
									SNew(STextBlock)
										.Text(LOCTEXT("SelectLabel", "Select:"))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.0f, 0.0f)
								[
									// all maps hyper link
									SNew(SHyperlink)
										.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllMapsHyperlinkNavigate, true)
										.Text(LOCTEXT("AllMapsHyperlinkLabel", "All"))
										.ToolTipText(LOCTEXT("AllMapsHyperlinkTooltip", "Select all available maps."))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									// no maps hyper link
									SNew(SHyperlink)
										.OnNavigate(this, &SProjectLauncherCookByTheBookSettings::HandleAllMapsHyperlinkNavigate, false)
										.Text(LOCTEXT("NoMapsHyperlinkLabel", "None"))
										.ToolTipText(LOCTEXT("NoMapsHyperlinkTooltip", "Deselect all maps."))
										.Visibility(this, &SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility)
								]
						]
				]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
				.InitiallyCollapsed(true)
				.Padding(8.0f)
				.BodyContent()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							// incremental cook check box
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("IncrementalCheckBoxTooltip", "If checked, only modified content will be cooked, resulting in much faster cooking times. It is recommended to enable this option whenever possible."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("IncrementalCheckBoxText", "Iterative cooking: Only cook content modified from previous cook"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// incremental cook check box
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UnversionedCheckBoxTooltip", "If checked, the version is assumed to be current at load. This is potentially dangerous, but results in smaller patch sizes."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("UnversionedCheckBoxText", "Save packages without versions"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// unreal pak check box
							SNew(SCheckBox)
								.IsChecked(this, &SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UnrealPakCheckBoxTooltip", "If checked, the content will be deployed as a single UnrealPak file instead of many separate files."))
								.Content()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("UnrealPakCheckBoxText", "Store all content in a single file (UnrealPak)"))
								]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 12.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookConfigurationSelectorLabel", "Cooker build configuration:"))							
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// cooker build configuration selector
							SNew(SProjectLauncherBuildConfigurationSelector)
								.OnConfigurationSelected(this, &SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorConfigurationSelected)
								.Text(this, &SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorText)
								.ToolTipText(LOCTEXT("CookConfigurationToolTipText", "Sets the build configuration to use for the cooker commandlet."))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 8.0f, 0.0f, 0.0f)
						[
							SNew(SProjectLauncherFormLabel)
								.LabelText(LOCTEXT("CookerOptionsTextBoxLabel", "Additional Cooker Options:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							// cooker command line options
							SNew(SEditableTextBox)
								.ToolTipText(LOCTEXT("CookerOptionsTextBoxTooltip", "Additional cooker command line parameters can be specified here."))
								.Text(this, &SProjectLauncherCookByTheBookSettings::HandleCookOptionsTextBlockText)
								.OnTextCommitted(this, &SProjectLauncherCookByTheBookSettings::HandleCookerOptionsCommitted)
						]
				]
		];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SProjectLauncherCookByTheBookSettings::RefreshCultureList()
{
	CultureList.Reset();

	TArray<FString> CultureNames;
	FInternationalization::Get().GetCultureNames(CultureNames);

	if (CultureNames.Num() > 0)
	{
		for (int32 Index = 0; Index < CultureNames.Num(); ++Index)
		{
			FString CultureName = CultureNames[Index];
			CultureList.Add(MakeShareable(new FString(CultureName)));
		}
	}

	if (CultureListView.IsValid())
	{
		CultureListView->RequestListRefresh();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherCookByTheBookSettings::RefreshMapList()
{
	MapList.Reset();

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		TArray<FString> AvailableMaps = FGameProjectHelper::GetAvailableMaps(SelectedProfile->GetProjectBasePath(), SelectedProfile->SupportsEngineMaps(), true);

		if (ShowMapsChoice == EShowMapsChoices::ShowMissingMaps)
		{
			for (const FString& Map : SelectedProfile->GetCookedMaps())
			{
				if (!AvailableMaps.Contains(Map))
				{
					MapList.Add(MakeShareable(new FString(Map)));
				}
			}
		}
		else
		{
			for (int32 AvailableMapIndex = 0; AvailableMapIndex < AvailableMaps.Num(); ++AvailableMapIndex)
			{
				FString& Map = AvailableMaps[AvailableMapIndex];

				if ((ShowMapsChoice == EShowMapsChoices::ShowAllMaps) || SelectedProfile->GetCookedMaps().Contains(Map))
				{
					MapList.Add(MakeShareable(new FString(Map)));
				}
			}
		}
	}

	MapListView->RequestListRefresh();
}


/* SProjectLauncherCookByTheBookSettings callbacks
 *****************************************************************************/

void SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkNavigate(bool AllPlatforms)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (AllPlatforms)
		{
			TArray<FString> CultureNames;
			FInternationalization::Get().GetCultureNames(CultureNames);

			for (int32 ExtensionIndex = 0; ExtensionIndex < CultureNames.Num(); ++ExtensionIndex)
			{
				SelectedProfile->AddCookedCulture(CultureNames[ExtensionIndex]);
			}
		}
		else
		{
			SelectedProfile->ClearCookedCultures();
		}
	}
}


EVisibility SProjectLauncherCookByTheBookSettings::HandleAllCulturesHyperlinkVisibility() const
{
	TArray<FString> CultureNames;
	FInternationalization::Get().GetCultureNames(CultureNames);

	if (CultureNames.Num() > 1)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherCookByTheBookSettings::HandleAllMapsHyperlinkNavigate(bool AllPlatforms)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (AllPlatforms)
		{
			TArray<FString> AvailableMaps = FGameProjectHelper::GetAvailableMaps(SelectedProfile->GetProjectBasePath(), SelectedProfile->SupportsEngineMaps(), false);

			for (int32 AvailableMapIndex = 0; AvailableMapIndex < AvailableMaps.Num(); ++AvailableMapIndex)
			{
				SelectedProfile->AddCookedMap(AvailableMaps[AvailableMapIndex]);
			}
		}
		else
		{
			SelectedProfile->ClearCookedMaps();
		}
	}
}


EVisibility SProjectLauncherCookByTheBookSettings::HandleMapSelectionHyperlinkVisibility() const
{
	if (MapList.Num() > 1)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorConfigurationSelected(EBuildConfiguration Configuration)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCookConfiguration(Configuration);
	}
}


FText SProjectLauncherCookByTheBookSettings::HandleCookConfigurationSelectorText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(LexToString(SelectedProfile->GetCookConfiguration()));
	}

	return FText::GetEmpty();
}


void SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetIncrementalCooking(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleIncrementalCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCookingIncrementally())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleSharedCookedBuildCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetIterateSharedCookedBuild(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleSharedCookedBuildCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsIterateSharedCookedBuild())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleCompressedCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCompressed(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleCompressedCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCompressed())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleEncryptIniFilesCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetEncryptingIniFiles(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleEncryptIniFilesCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsEncryptingIniFiles())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


TSharedRef<ITableRow> SProjectLauncherCookByTheBookSettings::HandleMapListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SProjectLauncherMapListRow, Model.ToSharedRef())
		.MapName(InItem)
		.OwnerTableView(OwnerTable);
}


TSharedRef<ITableRow> SProjectLauncherCookByTheBookSettings::HandleCultureListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SProjectLauncherCultureListRow, Model.ToSharedRef())
		.CultureName(InItem)
		.OwnerTableView(OwnerTable);
}


EVisibility SProjectLauncherCookByTheBookSettings::HandleNoMapSelectedBoxVisibility() const
{
	if (MapList.Num() == 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherCookByTheBookSettings::HandleNoMapsTextBlockText() const
{
	if (MapList.Num() == 0)
	{
		if (ShowMapsChoice == EShowMapsChoices::ShowAllMaps)
		{
			return LOCTEXT("NoMapsFoundText", "No available maps were found.");
		}
		else if (ShowMapsChoice == EShowMapsChoices::ShowCookedMaps)
		{
			return LOCTEXT("NoMapsSelectedText", "No map selected. Only startup packages will be cooked!");
		}
		else if (ShowMapsChoice == EShowMapsChoices::ShowMissingMaps)
		{
			return LOCTEXT("NoMissingMapsFoundText", "No missing maps were found.");
		}
	}

	return FText();
}


void SProjectLauncherCookByTheBookSettings::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	if (PreviousProfile.IsValid())
	{
		PreviousProfile->OnProjectChanged().RemoveAll(this);
	}
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->OnProjectChanged().AddSP(this, &SProjectLauncherCookByTheBookSettings::HandleProfileProjectChanged);
	}
	RefreshMapList();
	RefreshCultureList();
}


void SProjectLauncherCookByTheBookSettings::HandleProfileProjectChanged()
{
	RefreshMapList();
	RefreshCultureList();
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxIsChecked(EShowMapsChoices::Type Choice) const
{
	if (ShowMapsChoice == Choice)
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleShowCheckBoxCheckStateChanged(ECheckBoxState NewState, EShowMapsChoices::Type Choice)
{
	if (NewState == ECheckBoxState::Checked)
	{
		ShowMapsChoice = Choice;
		RefreshMapList();
	}
}


void SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetUnversionedCooking((NewState == ECheckBoxState::Checked));
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleUnversionedCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCookingUnversioned())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


EVisibility SProjectLauncherCookByTheBookSettings::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}


FText SProjectLauncherCookByTheBookSettings::HandleCookOptionsTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetCookOptions());
	}

	return result;
}


void SProjectLauncherCookByTheBookSettings::HandleCookerOptionsCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		FString useOptions = NewText.ToString();
		switch (CommitType)
		{
		case ETextCommit::Default:
		case ETextCommit::OnCleared:
			useOptions = TEXT("");
			break;
		default:
			break;
		}
		SelectedProfile->SetCookOptions(useOptions);
	}
}


void SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetDeployWithUnrealPak(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleUnrealPakCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsPackingWithUnrealPak())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleGeneratePatchCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetGeneratePatch(NewState == ECheckBoxState::Checked);
	}
}


void SProjectLauncherCookByTheBookSettings::HandleAddPatchLevelCheckBoxCheckStateChanged( ECheckBoxState NewState )
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetAddPatchLevel(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleGeneratePatchCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsGeneratingPatch())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

// Callback for determining the checked state of the 'AddPatchLevel' check box.
ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleAddPatchLevelCheckBoxIsChecked( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->ShouldAddPatchLevel())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCreateReleaseVersion(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCreatingReleaseVersion())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


FText SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionNameTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetCreateReleaseVersionName());
	}

	return result;
}


void SProjectLauncherCookByTheBookSettings::HandleCreateReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCreateReleaseVersionName(NewText.ToString());
	}

}


FText SProjectLauncherCookByTheBookSettings::HandleBasedOnReleaseVersionNameTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetBasedOnReleaseVersionName());
	}

	return result;
}


void SProjectLauncherCookByTheBookSettings::HandleBasedOnReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetBasedOnReleaseVersionName(NewText.ToString());
	}

}


FText SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionNameTextBlockText() const
{
	const ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetOriginalReleaseVersionName());
	}

	return result;
}


void SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetOriginalReleaseVersionName(NewText.ToString());
	}
}


EVisibility SProjectLauncherCookByTheBookSettings::HandleOriginalReleaseVersionVisibility() const
{
	const ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	const TArray<FString>& SelectedCookedPlatformNames = SelectedProfile->GetCookedPlatforms();
	const TArray<ITargetPlatform*>& Platforms = GetTargetPlatformManager()->GetTargetPlatforms();

	for (const FString& SelectedCookedPlatform : SelectedCookedPlatformNames)
	{
		for (const ITargetPlatform* Platform : Platforms)
		{
			if (SelectedCookedPlatform == Platform->PlatformName())
			{
				if (Platform->RequiresOriginalReleaseVersionForPatch())
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherCookByTheBookSettings::HandleDLCNameTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetDLCName());
	}

	return result;
}


void SProjectLauncherCookByTheBookSettings::HandleDLCNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetDLCName(NewText.ToString());
	}
}


void SProjectLauncherCookByTheBookSettings::HandleBuildDLCCheckBoxCheckStateChanged(ECheckBoxState NewState) 
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCreateDLC(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleBuildDLCCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCreatingDLC())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


FReply SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			LOCTEXT("RepositoryBrowseTitle", "Choose a repository location").ToString(),
			HttpChunkInstallDirectoryTextBox->GetText().ToString(),
			FolderName
			);

		if (bFolderSelected)
		{
			if (!FolderName.EndsWith(TEXT("/")))
			{
				FolderName += TEXT("/");
			}

			HttpChunkInstallDirectoryTextBox->SetText(FText::FromString(FolderName));
			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

			if (SelectedProfile.IsValid())
			{
				SelectedProfile->SetHttpChunkDataDirectory(FolderName);
			}
		}
	}

	return FReply::Handled();
}


FText SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(SelectedProfile->GetHttpChunkDataDirectory());
	}

	return FText::GetEmpty();
}


void SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryTextChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetHttpChunkDataDirectory(InText.ToString());
	}
}


void SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetHttpChunkDataDirectory(InText.ToString());
		}
	}
}


void SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetGenerateHttpChunkData(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->IsGenerateHttpChunkData() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}


FText SProjectLauncherCookByTheBookSettings::HandleHttpChunkInstallNameTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(SelectedProfile->GetHttpChunkDataReleaseName());
	}

	return FText();
}


void SProjectLauncherCookByTheBookSettings::HandleHtppChunkInstallNameCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetHttpChunkDataReleaseName(NewText.ToString());
		}
	}
}


void SProjectLauncherCookByTheBookSettings::HandleDLCIncludeEngineContentCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetDLCIncludeEngineContent(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleDLCIncludeEngineContentCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsDLCIncludingEngineContent())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleGenerateChunksCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetGenerateChunks(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleGenerateChunksCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->IsGeneratingChunks() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookByTheBookSettings::HandleDontIncludeEditorContentCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetSkipCookingEditorContent(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookByTheBookSettings::HandleDontIncludeEditorContentCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->GetSkipCookingEditorContent() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
