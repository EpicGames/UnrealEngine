// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class Error;
class SEditableTextBox;

namespace EShowMapsChoices
{
	enum Type
	{
		/** Show all available maps. */
		ShowAllMaps,

		/** Only show maps that are to be cooked. */
		ShowCookedMaps,

		/** Only show maps that are to be cooked but missing. */
		ShowMissingMaps,
	};
}


/**
 * Implements the cook-by-the-book settings panel.
 */
class SProjectLauncherCookByTheBookSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherCookByTheBookSettings) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SProjectLauncherCookByTheBookSettings();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowSimple = false);

protected:

	/** Refreshes the list of available maps. */
	void RefreshMapList();

	/** Refreshes the list of available cultures. */
	void RefreshCultureList();

private:

	/** Handles clicking the 'Select All Cultures' button. */
	void HandleAllCulturesHyperlinkNavigate(bool AllPlatforms);

	/** Handles determining the visibility of the 'Select All Cultures' button. */
	EVisibility HandleAllCulturesHyperlinkVisibility() const;

	/** Handles clicking the 'Select All Maps' button. */
	void HandleAllMapsHyperlinkNavigate(bool AllPlatforms);

	/** Handles selecting a build configuration for the cooker. */
	void HandleCookConfigurationSelectorConfigurationSelected(EBuildConfiguration);

	/** Handles getting the content text of the cooker build configuration selector. */
	FText HandleCookConfigurationSelectorText() const;

	/** Handles check state changes of the 'Incremental' check box. */
	void HandleIncrementalCheckBoxCheckStateChanged(ECheckBoxState NewState);

	/** Handles determining the checked state of the 'Incremental' check box. */
	ECheckBoxState HandleIncrementalCheckBoxIsChecked() const;
	
	void HandleSharedCookedBuildCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleSharedCookedBuildCheckBoxIsChecked() const;


	void HandleCompressedCheckBoxCheckStateChanged(ECheckBoxState NewState);
	void HandleEncryptIniFilesCheckBoxCheckStateChanged(ECheckBoxState NewState);

	ECheckBoxState HandleCompressedCheckBoxIsChecked() const;
	ECheckBoxState HandleEncryptIniFilesCheckBoxIsChecked() const;
	
	
	
	
	/** Handles generating a row widget in the map list view. */
	TSharedRef<ITableRow> HandleMapListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handles generating a row widget in the culture list view. */
	TSharedRef<ITableRow> HandleCultureListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handles determining the visibility of the 'Select All Maps' button. */
	EVisibility HandleMapSelectionHyperlinkVisibility() const;

	/** Handles getting the visibility of the map selection warning message. */
	EVisibility HandleNoMapSelectedBoxVisibility() const;

	/** Handles getting the text in the 'No maps' text block. */
	FText HandleNoMapsTextBlockText() const;

	/** Handles changing the selected profile in the profile manager. */
	void HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile);

	/** Handles checking the specified 'Show maps' check box. */
	void HandleShowCheckBoxCheckStateChanged(ECheckBoxState NewState, EShowMapsChoices::Type Choice);

	/** Handles determining the checked state of the specified 'Show maps' check box. */
	ECheckBoxState HandleShowCheckBoxIsChecked(EShowMapsChoices::Type Choice) const;

	/** Handles checking the specified 'Unversioned' check box. */
	void HandleUnversionedCheckBoxCheckStateChanged(ECheckBoxState NewState);

	/** Handles determining the checked state of the specified 'Unversioned' check box. */
	ECheckBoxState HandleUnversionedCheckBoxIsChecked() const;

	/** Handles determining the visibility of a validation error icon. */
	EVisibility HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const;

	/** Callback for getting the cookers additional options. */
	FText HandleCookOptionsTextBlockText() const;
	/** Callback for changing the cookers additional options */
	void HandleCookerOptionsCommitted(const FText& NewText, ETextCommit::Type CommitType);

	/** Callback for updating any settings after the selected project has changed in the profile. */
	void HandleProfileProjectChanged();

	// Callback for check state changes of the 'UnrealPak' check box.
	void HandleUnrealPakCheckBoxCheckStateChanged(ECheckBoxState NewState);

	// Callback for determining the checked state of the 'UnrealPak' check box.
	ECheckBoxState HandleUnrealPakCheckBoxIsChecked() const;

	void HandleGenerateChunksCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleGenerateChunksCheckBoxIsChecked() const;

	// callbacks for including editor content checkbox
	void HandleDontIncludeEditorContentCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleDontIncludeEditorContentCheckBoxIsChecked() const;



	//////////////////////////////////////////////////////////////////////////
	// creating release version related functions
	// Callback for check state changes of the 'GeneratePatch' check box.
	void HandleCreateReleaseVersionCheckBoxCheckStateChanged(ECheckBoxState NewState);
	// Callback for determining the checked state of the 'GeneratePatch' check box.
	ECheckBoxState HandleCreateReleaseVersionCheckBoxIsChecked() const;

	void HandleCreateReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	FText HandleCreateReleaseVersionNameTextBlockText() const;

	void HandleBasedOnReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	FText HandleBasedOnReleaseVersionNameTextBlockText() const;

	void HandleOriginalReleaseVersionNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	FText HandleOriginalReleaseVersionNameTextBlockText() const;
	EVisibility HandleOriginalReleaseVersionVisibility() const;

	//////////////////////////////////////////////////////////////////////////
	// patch generation related functions
	// Callback for check state changes of the 'GeneratePatch' check box.
	void HandleGeneratePatchCheckBoxCheckStateChanged(ECheckBoxState NewState);
	// Callback for determining the checked state of the 'GeneratePatch' check box.
	ECheckBoxState HandleGeneratePatchCheckBoxIsChecked() const;

	// Callback for check state changes of the 'AddPatchLevel' check box.
	void HandleAddPatchLevelCheckBoxCheckStateChanged( ECheckBoxState NewState );
	// Callback for determining the checked state of the 'AddPatchLevel' check box.
	ECheckBoxState HandleAddPatchLevelCheckBoxIsChecked( ) const;

	// Callback for changing patchSourceContent path (should be the path of a pak file)
	FText HandlePatchSourceContentPathTextBlockText() const;
	// Callback for getting the PatchSourceContentPath
	void HandlePatchSourceContentPathCommitted(const FText& NewText, ETextCommit::Type CommitType);

	//////////////////////////////////////////////////////////////////////////
	// dlc check box related functions 
	void HandleBuildDLCCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleBuildDLCCheckBoxIsChecked() const;
	void HandleDLCNameCommitted(const FText& NewText, ETextCommit::Type CommitType);
	FText HandleDLCNameTextBlockText() const;
	void HandleDLCIncludeEngineContentCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleDLCIncludeEngineContentCheckBoxIsChecked() const;

	//////////////////////////////////////////////////////////////////////////
	// Http Chunk Installer options & functions
	FReply HandleHtppChunkInstallBrowseButtonClicked();
	FText HandleHtppChunkInstallDirectoryText() const;
	void HandleHtppChunkInstallDirectoryTextChanged(const FText& InText);
	void HandleHtppChunkInstallDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	void HandleHttpChunkInstallCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleHttpChunkInstallCheckBoxIsChecked() const;
	FText HandleHttpChunkInstallNameTextBlockText() const;
	void HandleHtppChunkInstallNameCommitted(const FText& NewText, ETextCommit::Type CommitType);

	/** creates the complex widget. */
	TSharedRef<SWidget> MakeComplexWidget();

	/** creates the simple widget. */
	TSharedRef<SWidget> MakeSimpleWidget();

private:

	/** Textbox which holds the PatchSourceContentPath */
	TSharedPtr<SEditableTextBox> PatchSourceContentPath;

	/** Textbox which holds the DLCBasedOnReleaseVersion */
	TSharedPtr<SEditableTextBox> DLCBasedOnReleaseVersionName;

	/** Holds the culture list. */
	TArray<TSharedPtr<FString> > CultureList;

	/** Holds the map list view. */
	TSharedPtr<SListView<TSharedPtr<FString> > > CultureListView;

	/** Holds the map list. */
	TArray<TSharedPtr<FString> > MapList;

	/** Holds the map list view. */
	TSharedPtr<SListView<TSharedPtr<FString> > > MapListView;

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** Holds the current 'Show maps' check box choice. */
	EShowMapsChoices::Type ShowMapsChoice;

	// Holds the Http Chunk Install directory path text box.
	TSharedPtr<SEditableTextBox> HttpChunkInstallDirectoryTextBox;
};
