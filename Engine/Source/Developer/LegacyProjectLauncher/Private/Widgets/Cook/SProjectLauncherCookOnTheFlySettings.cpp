// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Cook/SProjectLauncherCookOnTheFlySettings.h"

#include "SlateOptMacros.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Shared/SProjectLauncherFormLabel.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherCookOnTheFlySettings"


/* SProjectLauncherCookOnTheFlySettings structors
 *****************************************************************************/

SProjectLauncherCookOnTheFlySettings::~SProjectLauncherCookOnTheFlySettings()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherCookOnTheFlySettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherCookOnTheFlySettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								// incremental cook check box
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherCookOnTheFlySettings::HandleIncrementalCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherCookOnTheFlySettings::HandleIncrementalCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("IncrementalCheckBoxTooltip", "If checked, only modified content will be cooked, resulting in much faster cooking times. It is recommended to enable this option whenever possible."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("IncrementalCheckBoxText", "Only cook modified content"))
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 12.0f, 0.0f, 0.0f)
							[
								SNew(SProjectLauncherFormLabel)
									.LabelText(LOCTEXT("CookerOptionsTextBoxLabel", "Additional Cooker Options:"))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								// cooker command line options
								SNew(SEditableTextBox)
								.ToolTipText(LOCTEXT("CookerOptionsTextBoxTooltip", "Additional cooker command line parameters can be specified here."))
								.Text(this, &SProjectLauncherCookOnTheFlySettings::HandleCookOptionsTextBlockText)
								.OnTextCommitted(this, &SProjectLauncherCookOnTheFlySettings::HandleCookerOptionsCommitted)
							]
					]
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherCookOnTheFlySettings::HandleProfileManagerProfileSelected);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherCookOnTheFlySettings callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherCookOnTheFlySettings::HandleIncrementalCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetIncrementalCooking(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherCookOnTheFlySettings::HandleIncrementalCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsCookingIncrementally())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherCookOnTheFlySettings::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{

}


EVisibility SProjectLauncherCookOnTheFlySettings::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}


FText SProjectLauncherCookOnTheFlySettings::HandleCookOptionsTextBlockText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	FText result;

	if (SelectedProfile.IsValid())
	{
		result = FText::FromString(SelectedProfile->GetCookOptions());
	}

	return result;
}


void SProjectLauncherCookOnTheFlySettings::HandleCookerOptionsCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		FString useOptions = NewText.ToString();
		switch (CommitType)
		{
		case ETextCommit::Default:
		case ETextCommit::OnCleared:
			useOptions = TEXT("");
			break;

		default:
			break;
		}
		SelectedProfile->SetCookOptions(useOptions);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
