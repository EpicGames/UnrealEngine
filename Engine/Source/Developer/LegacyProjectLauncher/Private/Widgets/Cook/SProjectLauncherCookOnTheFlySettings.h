// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class Error;
enum class ECheckBoxState : uint8;

/**
 * Implements the cook-by-the-book settings panel.
 */
class SProjectLauncherCookOnTheFlySettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherCookOnTheFlySettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherCookOnTheFlySettings( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	// Callback for check state changes of the 'Incremental' check box.
	void HandleIncrementalCheckBoxCheckStateChanged( ECheckBoxState NewState );

	// Callback for determining the checked state of the 'Incremental' check box.
	ECheckBoxState HandleIncrementalCheckBoxIsChecked( ) const;

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

	// Callback for determining the visibility of a validation error icon.
	EVisibility HandleValidationErrorIconVisibility( ELauncherProfileValidationErrors::Type Error ) const;

	// Callback for getting the cookers additional options
	FText HandleCookOptionsTextBlockText() const;

	// Callback for changing the cookers additional options
	void HandleCookerOptionsCommitted(const FText& NewText, ETextCommit::Type CommitType);

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
