// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Cook/SProjectLauncherCookPage.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Cook/SProjectLauncherCookByTheBookSettings.h"
#include "Widgets/Cook/SProjectLauncherCookOnTheFlySettings.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherCookPage"


/* SProjectLauncherCookPage structors
 *****************************************************************************/

SProjectLauncherCookPage::~SProjectLauncherCookPage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherCookPage interface
 *****************************************************************************/

void SProjectLauncherCookPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	// create cook modes menu
	FMenuBuilder CookModeMenuBuilder(true, NULL);
	{
		FUIAction ByTheBookAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookPage::HandleCookModeMenuEntryClicked, ELauncherProfileCookModes::ByTheBook));
		CookModeMenuBuilder.AddMenuEntry(LOCTEXT("ByTheBookAction", "By the book"), LOCTEXT("ByTheBookActionHint", "Specify which content should be cooked and cook everything in advance prior to launching the game."), FSlateIcon(), ByTheBookAction);
		
		FUIAction OnTheFlyAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookPage::HandleCookModeMenuEntryClicked, ELauncherProfileCookModes::OnTheFly));
		CookModeMenuBuilder.AddMenuEntry(LOCTEXT("OnTheFlyAction", "On the fly"), LOCTEXT("OnTheFlyActionHint", "Cook the content at run-time before it is being sent to the device."), FSlateIcon(), OnTheFlyAction);

		FUIAction DoNotCookAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookPage::HandleCookModeMenuEntryClicked, ELauncherProfileCookModes::DoNotCook));
		CookModeMenuBuilder.AddMenuEntry(LOCTEXT("DoNotCookAction", "Do not cook"), LOCTEXT("DoNotCookActionHint", "Do not cook the content at this time."), FSlateIcon(), DoNotCookAction);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("HowToCookText", "How would you like to cook the content?"))
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0, 0.0, 0.0, 0.0)
					[
						// cooking mode menu
						SNew(SComboButton)
							.ButtonContent()
							[
								SNew(STextBlock)
									.Text(this, &SProjectLauncherCookPage::HandleCookModeComboButtonContentText)
							]
							.ContentPadding(FMargin(6.0, 2.0))
							.MenuContent()
							[
								CookModeMenuBuilder.MakeWidget()
							]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SProjectLauncherCookOnTheFlySettings, InModel)
					.Visibility(this, &SProjectLauncherCookPage::HandleCookOnTheFlySettingsVisibility)
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SProjectLauncherCookByTheBookSettings, InModel)
					.Visibility(this, &SProjectLauncherCookPage::HandleCookByTheBookSettingsVisibility)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherCookPage::HandleProfileManagerProfileSelected);
}


/* SProjectLauncherCookPage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

EVisibility SProjectLauncherCookPage::HandleCookByTheBookSettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::ByTheBook)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherCookPage::HandleCookModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfileCookModes::Type CookMode = SelectedProfile->GetCookMode();

		if (CookMode == ELauncherProfileCookModes::ByTheBook)
		{
			return LOCTEXT("CookModeComboButton_ByTheBook", "By the book");
		}

		if (CookMode == ELauncherProfileCookModes::DoNotCook)
		{
			return LOCTEXT("CookModeComboButton_DoNotCook", "Do not cook");
		}

		if (CookMode == ELauncherProfileCookModes::OnTheFly)
		{
			return LOCTEXT("CookModeComboButton_OnTheFly", "On the fly");
		}

		return LOCTEXT("CookModeComboButtonDefaultText", "Select...");
	}

	return FText();
}


void SProjectLauncherCookPage::HandleCookModeMenuEntryClicked(ELauncherProfileCookModes::Type CookMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCookMode(CookMode);
	}
}


EVisibility SProjectLauncherCookPage::HandleCookOnTheFlySettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::OnTheFly)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherCookPage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	// reload settings
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
