// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Cook/SProjectLauncherCookedPlatforms.h"

#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Cook/SProjectLauncherPlatformListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherCookedPlatforms"


void SProjectLauncherCookedPlatforms::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	MakePlatformMenu();

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					// platform menu
					SAssignNew(PlatformListView, SListView<TSharedPtr<FString> >)
						.HeaderRow
						(
							SNew(SHeaderRow)
								.Visibility(EVisibility::Collapsed)

							+ SHeaderRow::Column("PlatformName")
								.DefaultLabel(LOCTEXT("PlatformListPlatformNameColumnHeader", "Platform"))
								.FillWidth(1.0f)
						)
						.ListItemsSource(&PlatformList)
						.OnGenerateRow(this, &SProjectLauncherCookedPlatforms::HandlePlatformListViewGenerateRow)
						.SelectionMode(ESelectionMode::None)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 6.0f, 0.0f, 4.0f)
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("SelectLabel", "Select:"))
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(8.0f, 0.0f)
						[
							// all platforms hyper-link
							SNew(SHyperlink)
								.OnNavigate(this, &SProjectLauncherCookedPlatforms::HandleAllPlatformsHyperlinkNavigate, true)
								.Text(LOCTEXT("AllPlatformsHyperlinkLabel", "All"))
								.ToolTipText(LOCTEXT("AllPlatformsButtonTooltip", "Select all available platforms."))
								.Visibility(this, &SProjectLauncherCookedPlatforms::HandleAllPlatformsHyperlinkVisibility)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							// no platforms hyper-link
							SNew(SHyperlink)
								.OnNavigate(this, &SProjectLauncherCookedPlatforms::HandleAllPlatformsHyperlinkNavigate, false)
								.Text(LOCTEXT("NoPlatformsHyperlinkLabel", "None"))
								.ToolTipText(LOCTEXT("NoPlatformsHyperlinkTooltip", "Deselect all platforms."))
								.Visibility(this, &SProjectLauncherCookedPlatforms::HandleAllPlatformsHyperlinkVisibility)
						]
				]
		];
}


TSharedRef<ITableRow> SProjectLauncherCookedPlatforms::HandlePlatformListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SProjectLauncherPlatformListRow, Model.ToSharedRef())
		.PlatformName(InItem)
		.OwnerTableView(OwnerTable);
}


#undef LOCTEXT_NAMESPACE
