// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Styling/AppStyle.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"


/**
 * Implements the cooked platforms panel.
 */
class SProjectLauncherCookedPlatforms
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherCookedPlatforms) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/**
	 * Build the platform menu.
	 *
	 * @return Platform menu widget.
	 */
	void MakePlatformMenu()
	{
		TArray<ITargetPlatform*> Platforms = GetTargetPlatformManager()->GetTargetPlatforms();

		if (Platforms.Num() > 0)
		{
			PlatformList.Reset();
			for (int32 PlatformIndex = 0; PlatformIndex < Platforms.Num(); ++PlatformIndex)
			{
				FString PlatformName = Platforms[PlatformIndex]->PlatformName();

				PlatformList.Add(MakeShareable(new FString(PlatformName)));
			}
		}
	}

private:

	/** Callback for clicking the 'Select All Platforms' button. */
	void HandleAllPlatformsHyperlinkNavigate(bool AllPlatforms)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AllPlatforms)
			{
				TArray<ITargetPlatform*> Platforms = GetTargetPlatformManager()->GetTargetPlatforms();

				for (int32 PlatformIndex = 0; PlatformIndex < Platforms.Num(); ++PlatformIndex)
				{
					SelectedProfile->AddCookedPlatform(Platforms[PlatformIndex]->PlatformName());
				}
			}
			else
			{
				SelectedProfile->ClearCookedPlatforms();
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	/** Callback for determining the visibility of the 'Select All Platforms' button. */
	EVisibility HandleAllPlatformsHyperlinkVisibility() const
	{
		if (GetTargetPlatformManager()->GetTargetPlatforms().Num() > 1)
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	}

	/** Callback for getting the color of a platform menu check box. */
	FSlateColor HandlePlatformMenuEntryColorAndOpacity(FString PlatformName) const
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);

			if (TargetPlatform != NULL)
			{
//				if (TargetPlatform->HasValidBuild(SelectedProfile->GetProjectPath(), SelectedProfile->GetBuildConfiguration()))
				{
					return FAppStyle::GetColor("Colors.Foreground");
				}
			}
		}

		return FLinearColor::Yellow;
	}

	/** Callback for generating a row widget in the map list view. */
	TSharedRef<ITableRow> HandlePlatformListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:

	/** Pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** The platform list. */
	TArray<TSharedPtr<FString> > PlatformList;

	/** The platform list view. */
	TSharedPtr<SListView<TSharedPtr<FString>>> PlatformListView;

};

