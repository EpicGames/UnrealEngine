// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherMapListRow"

/**
 * Implements a row widget for map list.
 */
class SProjectLauncherMapListRow
	: public SMultiColumnTableRow<TSharedPtr<FString> >
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherMapListRow) { }
		SLATE_ATTRIBUTE(FString, HighlightString)
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(TSharedPtr<FString>, MapName)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InProfileManager The profile manager to use.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel )
	{
		HighlightString = InArgs._HighlightString;
		MapName = InArgs._MapName;
		Model = InModel;

		SMultiColumnTableRow<TSharedPtr<FString> >::Construct(FSuperRowType::FArguments(), InArgs._OwnerTableView.ToSharedRef());
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "MapName")
		{
			return SNew(SCheckBox)
				.IsChecked(this, &SProjectLauncherMapListRow::HandleCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SProjectLauncherMapListRow::HandleCheckBoxCheckStateChanged)
				.Padding(FMargin(6.0, 2.0))
				[
					SNew(STextBlock)
						.Text(FText::FromString(*MapName))
				];
		}

		return SNullWidget::NullWidget;
	}

private:

PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// Callback for changing the checked state of the check box.
	void HandleCheckBoxCheckStateChanged( ECheckBoxState NewState )
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			if (NewState == ECheckBoxState::Checked)
			{
				SelectedProfile->AddCookedMap(*MapName);
			}
			else
			{
				SelectedProfile->RemoveCookedMap(*MapName);
			}
		}
	}

	// Callback for determining the checked state of the check box.
	ECheckBoxState HandleCheckBoxIsChecked( ) const
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid() && SelectedProfile->GetCookedMaps().Contains(*MapName))
		{
			return ECheckBoxState::Checked;
		}

		return ECheckBoxState::Unchecked;
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	// Holds the highlight string for the log message.
	TAttribute<FString> HighlightString;

	// Holds the map's name.
	TSharedPtr<FString> MapName;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};


#undef LOCTEXT_NAMESPACE
