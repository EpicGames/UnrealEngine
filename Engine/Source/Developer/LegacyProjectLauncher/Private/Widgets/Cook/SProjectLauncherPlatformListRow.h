// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherPlatformListRow"

/**
 * Implements a row widget for map list.
 */
class SProjectLauncherPlatformListRow
	: public SMultiColumnTableRow<TSharedPtr<FString> >
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherPlatformListRow) { }
		SLATE_ATTRIBUTE(FString, HighlightString)
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(TSharedPtr<FString>, PlatformName)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InProfileManager The profile manager to use.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel )
	{
		HighlightString = InArgs._HighlightString;
		PlatformName = InArgs._PlatformName;
		Model = InModel;

		SMultiColumnTableRow<TSharedPtr<FString> >::Construct(FSuperRowType::FArguments(), InArgs._OwnerTableView.ToSharedRef());
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "PlatformName")
		{
			return SNew(SCheckBox)
				.IsChecked(this, &SProjectLauncherPlatformListRow::HandleCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SProjectLauncherPlatformListRow::HandleCheckBoxCheckStateChanged)
				.Padding(FMargin(6.0, 2.0))
				[
					SNew(STextBlock)
						.Text(FText::FromString(*PlatformName))
				];
		}

		return SNullWidget::NullWidget;
	}

private:

PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// Callback for changing the checked state of the check box.
	void HandleCheckBoxCheckStateChanged( ECheckBoxState NewState )
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			if (NewState == ECheckBoxState::Checked)
			{
				SelectedProfile->AddCookedPlatform(*PlatformName);
			}
			else
			{
				SelectedProfile->RemoveCookedPlatform(*PlatformName);
			}
		}
	}

	// Callback for determining the checked state of the check box.
	ECheckBoxState HandleCheckBoxIsChecked( ) const
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			SelectedProfile->IsValidForLaunch();
			if (SelectedProfile->GetCookedPlatforms().Contains(*PlatformName))
			{
				return ECheckBoxState::Checked;
			}
		}

		return ECheckBoxState::Unchecked;
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	// Holds the highlight string for the log message.
	TAttribute<FString> HighlightString;

	// Holds the platform's name.
	TSharedPtr<FString> PlatformName;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};


#undef LOCTEXT_NAMESPACE
