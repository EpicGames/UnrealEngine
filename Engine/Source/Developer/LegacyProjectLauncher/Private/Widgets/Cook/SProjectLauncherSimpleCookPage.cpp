// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Cook/SProjectLauncherSimpleCookPage.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Cook/SProjectLauncherCookByTheBookSettings.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherCookPage"


/* SProjectLauncherSimpleCookPage structors
 *****************************************************************************/

SProjectLauncherSimpleCookPage::~SProjectLauncherSimpleCookPage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherSimpleCookPage interface
 *****************************************************************************/

void SProjectLauncherSimpleCookPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SProjectLauncherCookByTheBookSettings, InModel, true)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherSimpleCookPage::HandleProfileManagerProfileSelected);
}


/* SProjectLauncherSimpleCookPage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

EVisibility SProjectLauncherSimpleCookPage::HandleCookByTheBookSettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::ByTheBook)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherSimpleCookPage::HandleCookModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfileCookModes::Type CookMode = SelectedProfile->GetCookMode();

		if (CookMode == ELauncherProfileCookModes::ByTheBook)
		{
			return LOCTEXT("CookModeComboButton_ByTheBook", "By the book");
		}

		if (CookMode == ELauncherProfileCookModes::DoNotCook)
		{
			return LOCTEXT("CookModeComboButton_DoNotCook", "Do not cook");
		}

		if (CookMode == ELauncherProfileCookModes::OnTheFly)
		{
			return LOCTEXT("CookModeComboButton_OnTheFly", "On the fly");
		}

		return LOCTEXT("CookModeComboButtonDefaultText", "Select...");
	}

	return FText();
}


void SProjectLauncherSimpleCookPage::HandleCookModeMenuEntryClicked(ELauncherProfileCookModes::Type CookMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetCookMode(CookMode);
	}
}


EVisibility SProjectLauncherSimpleCookPage::HandleCookOnTheFlySettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::OnTheFly)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherSimpleCookPage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	// reload settings
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
