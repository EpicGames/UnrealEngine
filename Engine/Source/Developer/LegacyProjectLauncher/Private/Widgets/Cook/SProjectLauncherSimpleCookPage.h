// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class SEditableTextBox;

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherSimpleCookPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherSimpleCookPage) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherSimpleCookPage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	// Callback for getting the visibility of the cook-by-the-book settings area.
	EVisibility HandleCookByTheBookSettingsVisibility( ) const;

	// Callback for getting the content text of the 'Cook Mode' combo button.
	FText HandleCookModeComboButtonContentText( ) const;

	// Callback for clicking an item in the 'Cook Mode' menu.
	void HandleCookModeMenuEntryClicked( ELauncherProfileCookModes::Type CookMode );

	// Callback for getting the visibility of the cook-on-the-fly settings area.
	EVisibility HandleCookOnTheFlySettingsVisibility( ) const;

	// Callback for changing the list of profiles in the profile manager.
	void HandleProfileManagerProfileListChanged( );

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

private:

	// Holds the cooker options text box.
	TSharedPtr<SEditableTextBox> CookerOptionsTextBox;

	// Holds the list of available cook modes.
	TArray<TSharedPtr<FString> > CookModeList;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
