// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeployFileServerSettings.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Deploy/SProjectLauncherDeployTargets.h"
#include "Widgets/Layout/SExpandableArea.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherDeployFileServerSettings"


/* SProjectLauncherDeployFileServerSettings interface
 *****************************************************************************/

void SProjectLauncherDeployFileServerSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.Padding(8.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						// deploy targets area
						SNew(SProjectLauncherDeployTargets, InModel)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								// hide file server check box
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherDeployFileServerSettings::HandleHideWindowCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherDeployFileServerSettings::HandleHideWindowCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("HideWindowCheckBoxTooltip", "If checked, the file server's console window will be hidden from your desktop."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("HideWindowCheckBoxText", "Hide the file server's console window"))
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								// streaming file server check box
								SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherDeployFileServerSettings::HandleStreamingServerCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherDeployFileServerSettings::HandleStreamingServerCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("StreamingServerCheckBoxTooltip", "If checked, the file server uses an experimental implementation that can serve multiple files simultaneously."))
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("StreamingServerCheckBoxText", "Streaming server (experimental)"))
									]
							]
					]
			]
	];
}


/* SProjectLauncherDeployFileServerSettings callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherDeployFileServerSettings::HandleHideWindowCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetHideFileServerWindow(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherDeployFileServerSettings::HandleHideWindowCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsFileServerHidden())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}


void SProjectLauncherDeployFileServerSettings::HandleStreamingServerCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetStreamingFileServer(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherDeployFileServerSettings::HandleStreamingServerCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsFileServerStreaming())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
