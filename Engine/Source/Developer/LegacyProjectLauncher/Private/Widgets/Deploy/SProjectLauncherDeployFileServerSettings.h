// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FArguments;
class FProjectLauncherModel;
enum class ECheckBoxState : uint8;


/**
 * Implements the deploy-to-device settings panel.
 */
class SProjectLauncherDeployFileServerSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployFileServerSettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

private:

	/** Callback for check state changes of the 'Hide Window' check box. */
	void HandleHideWindowCheckBoxCheckStateChanged(ECheckBoxState NewState);

	/** Callback for determining the checked state of the 'Hide Window' check box. */
	ECheckBoxState HandleHideWindowCheckBoxIsChecked() const;

	/** Callback for check state changes of the 'Streaming Server' check box. */
	void HandleStreamingServerCheckBoxCheckStateChanged(ECheckBoxState NewState);

	/** Callback for determining the checked state of the 'Streaming Server' check box. */
	ECheckBoxState HandleStreamingServerCheckBoxIsChecked() const;

private:

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;
};
