// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeployPage.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Deploy/SProjectLauncherDeployFileServerSettings.h"
#include "Widgets/Deploy/SProjectLauncherDeployToDeviceSettings.h"
#include "Widgets/Deploy/SProjectLauncherDeployRepositorySettings.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherDevicesPage"


/* SProjectLauncherProfilePage structors
 *****************************************************************************/

SProjectLauncherDeployPage::~SProjectLauncherDeployPage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherDevicesPage interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherDeployPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool IsFromRepository)
{
	Model = InModel;

	// create deployment mode menu
	FMenuBuilder DeploymentModeMenuBuilder(true, NULL);
	{
		FUIAction FileServerAction(FExecuteAction::CreateSP(this, &SProjectLauncherDeployPage::HandleDeploymentModeMenuEntryClicked, ELauncherProfileDeploymentModes::FileServer));
		DeploymentModeMenuBuilder.AddMenuEntry(LOCTEXT("FileServerAction", "File server"), LOCTEXT("FileServerActionHint", "Use a file server to deploy game content on the fly."), FSlateIcon(), FileServerAction);
		
		FUIAction CopyToDeviceAction(FExecuteAction::CreateSP(this, &SProjectLauncherDeployPage::HandleDeploymentModeMenuEntryClicked, ELauncherProfileDeploymentModes::CopyToDevice));
		DeploymentModeMenuBuilder.AddMenuEntry(LOCTEXT("CopyToDeviceAction", "Copy to device"), LOCTEXT("CopyToDeviceActionHint", "Copy the entire build to the device."), FSlateIcon(), CopyToDeviceAction);

		FUIAction DoNotDeployAction(FExecuteAction::CreateSP(this, &SProjectLauncherDeployPage::HandleDeploymentModeMenuEntryClicked, ELauncherProfileDeploymentModes::DoNotDeploy));
		DeploymentModeMenuBuilder.AddMenuEntry(LOCTEXT("DoNotDeployAction", "Do not deploy"), LOCTEXT("DoNotDeployActionHint", "Do not deploy the build at this time."), FSlateIcon(), DoNotDeployAction);

		FUIAction CopyRepositoryAction(FExecuteAction::CreateSP(this, &SProjectLauncherDeployPage::HandleDeploymentModeMenuEntryClicked, ELauncherProfileDeploymentModes::CopyRepository));
		DeploymentModeMenuBuilder.AddMenuEntry(LOCTEXT("CopyRepositoryAction", "Copy repository"), LOCTEXT("CopyRepositoryActionHint", "Copy a build from a repository to the device."), FSlateIcon(), CopyRepositoryAction);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility(IsFromRepository ? EVisibility::Hidden : EVisibility::Visible)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("HowToDeployText", "How would you like to deploy the build?"))			
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						// deployment mode menu
						SNew(SComboButton)
							.ButtonContent()
							[
								SNew(STextBlock)
									.Text(this, &SProjectLauncherDeployPage::HandleDeploymentModeComboButtonContentText)
							]
							.ContentPadding(FMargin(6.0f, 2.0f))
							.MenuContent()
							[
								DeploymentModeMenuBuilder.MakeWidget()
							]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(8.0f)
					.Visibility(this, &SProjectLauncherDeployPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SImage)
									.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("CopyToDeviceRequiresCookByTheBookText", "This mode requires 'By The Book' cooking"))
							]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				// file server settings area
				SNew(SProjectLauncherDeployFileServerSettings, InModel)
					.Visibility(this, &SProjectLauncherDeployPage::HandleDeployFileServerSettingsVisibility)
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				// deploy to devices settings area
				SNew(SProjectLauncherDeployToDeviceSettings, InModel)
					.Visibility(this, &SProjectLauncherDeployPage::HandleDeployToDeviceSettingsVisibility)
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				// deploy repository to devices settings area
				SNew(SProjectLauncherDeployRepositorySettings, InModel)
					.Visibility(this, &SProjectLauncherDeployPage::HandleDeployRepositorySettingsVisibility)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherDeployPage::HandleProfileManagerProfileSelected);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherDevicesPage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FText SProjectLauncherDeployPage::HandleDeploymentModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		switch (SelectedProfile->GetDeploymentMode())
		{
		case ELauncherProfileDeploymentModes::CopyToDevice:
			return LOCTEXT("CopyToDeviceAction", "Copy to device");
		case ELauncherProfileDeploymentModes::DoNotDeploy:
			return LOCTEXT("DoNotDeployAction", "Do not deploy");
		case ELauncherProfileDeploymentModes::FileServer:
			return LOCTEXT("FileServerAction", "File server");
		case ELauncherProfileDeploymentModes::CopyRepository:
			return LOCTEXT("CopyRepositoryAction", "Copy repository");
		default:
			return LOCTEXT("DeploymentModeComboButtonDefaultText", "Select...");
		}
	}

	return FText::GetEmpty();
}


void SProjectLauncherDeployPage::HandleDeploymentModeMenuEntryClicked(ELauncherProfileDeploymentModes::Type DeploymentMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetDeploymentMode(DeploymentMode);
	}
}


EVisibility SProjectLauncherDeployPage::HandleDeployFileServerSettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() == ELauncherProfileDeploymentModes::FileServer)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherDeployPage::HandleDeployToDeviceSettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyToDevice)
		{
			if (!SelectedProfile->HasValidationError(ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook))
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherDeployPage::HandleDeployRepositorySettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyRepository)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherDeployPage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{

}


EVisibility SProjectLauncherDeployPage::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
