// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class Error;

/**
 * Implements the devices page for the session launcher wizard.
 */
class SProjectLauncherDeployPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployPage) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherDeployPage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The Slate argument list.
	 * @param InModel - The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool IsFromRepository = false );

private:

	// Callback for getting the content text of the 'Deployment Mode' combo button.
	FText HandleDeploymentModeComboButtonContentText( ) const;

	// Callback for clicking an item in the 'Deployment Mode' menu.
	void HandleDeploymentModeMenuEntryClicked( ELauncherProfileDeploymentModes::Type DeploymentMode );

	// Callback for getting the visibility of the deployment targets area.
	EVisibility HandleDeployFileServerSettingsVisibility( ) const;

	// Callback for getting the visibility of the deployment targets area.
	EVisibility HandleDeployToDeviceSettingsVisibility( ) const;

	// Callback for getting the visibility of the deployment repository area.
	EVisibility HandleDeployRepositorySettingsVisibility( ) const;

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

	// Callback for determining the visibility of a validation error icon.
	EVisibility HandleValidationErrorIconVisibility( ELauncherProfileValidationErrors::Type Error ) const;

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
