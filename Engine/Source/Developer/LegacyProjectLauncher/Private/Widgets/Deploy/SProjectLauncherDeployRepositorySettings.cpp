// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeployRepositorySettings.h"

#include "Styling/AppStyle.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Deploy/SProjectLauncherDeployTargets.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherDeployRepositorySettings"


/* SProjectLauncherDeployRepositorySettings interface
 *****************************************************************************/

void SProjectLauncherDeployRepositorySettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			SNew(SBorder)
				.Padding(8.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
								.Text(LOCTEXT("RepositoryPathLabel", "Repository Path:"))
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0, 4.0, 0.0, 0.0)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.Padding(0.0, 0.0, 0.0, 3.0)
								[
									// repository path text box
									SAssignNew(RepositoryPathTextBox, SEditableTextBox)
										.OnTextCommitted(this, &SProjectLauncherDeployRepositorySettings::OnTextCommitted)
										.OnTextChanged(this, &SProjectLauncherDeployRepositorySettings::OnTextChanged)
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Right)
								.Padding(4.0, 0.0, 0.0, 0.0)
								[
									// browse button
									SNew(SButton)
										.ContentPadding(FMargin(6.0, 2.0))
										.IsEnabled(true)
										.Text(LOCTEXT("BrowseButtonText", "Browse..."))
										.ToolTipText(LOCTEXT("BrowseButtonToolTip", "Browse for the repository"))
										.OnClicked(this, &SProjectLauncherDeployRepositorySettings::HandleBrowseButtonClicked)
								]
						]
				]
		]

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.Padding(8.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						// deploy targets area
						SNew(SProjectLauncherDeployTargets, InModel)
					]
			]
	];
}


/* SProjectLauncherDeployRepositorySettings callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FReply SProjectLauncherDeployRepositorySettings::HandleBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			LOCTEXT("RepositoryBrowseTitle", "Choose a repository location").ToString(),
			RepositoryPathTextBox->GetText().ToString(),
			FolderName
			);

		if (bFolderSelected)
		{
			if (!FolderName.EndsWith(TEXT("/")))
			{
				FolderName += TEXT("/");
			}

			RepositoryPathTextBox->SetText(FText::FromString(FolderName));
			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

			if(SelectedProfile.IsValid())
			{
				SelectedProfile->SetPackageDirectory(FolderName);
			}
		}
	}

	return FReply::Handled();
}


void SProjectLauncherDeployRepositorySettings::OnTextChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if(SelectedProfile.IsValid())
	{
		SelectedProfile->SetPackageDirectory(InText.ToString());
	}
}


void SProjectLauncherDeployRepositorySettings::OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if(SelectedProfile.IsValid())
		{
			SelectedProfile->SetPackageDirectory(InText.ToString());
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
