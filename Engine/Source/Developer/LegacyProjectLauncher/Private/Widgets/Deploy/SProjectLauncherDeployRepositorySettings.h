// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class SEditableTextBox;

/**
 * Implements the deploy-to-device settings panel.
 */
class SProjectLauncherDeployRepositorySettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployRepositorySettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The Slate argument list.
	 * @param InModel - The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	FReply HandleBrowseButtonClicked( );

	/** Handles entering in a command */
	void OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	void OnTextChanged(const FText& InText);

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

	// Holds the repository path text box.
	TSharedPtr<SEditableTextBox> RepositoryPathTextBox;
};
