// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherDeviceGroup.h"
#include "ITargetDeviceProxy.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "PlatformInfo.h"
#include "Widgets/Shared/SProjectLauncherVariantSelector.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherDeployTargetListRow"

/**
 * Implements a row widget for the launcher's device proxy list.
 */
class SProjectLauncherDeployTargetListRow
	: public SMultiColumnTableRow<TSharedPtr<ITargetDeviceProxy>>
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployTargetListRow) { }
		
		/**
		 * The currently selected device group.
		 */
		SLATE_ATTRIBUTE(ILauncherDeviceGroupPtr, DeviceGroup)

		/**
		 * The device proxy shown in this row.
		 */
		SLATE_ARGUMENT(TSharedPtr<ITargetDeviceProxy>, DeviceProxy)

		/**
		 * The row's highlight text.
		 */
		SLATE_ATTRIBUTE(FText, HighlightText)

	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The construction arguments.
	 * @param InDeviceGroup - The device group that is being edited.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		DeviceGroup = InArgs._DeviceGroup;
		DeviceProxy = InArgs._DeviceProxy;
		HighlightText = InArgs._HighlightText;

		SelectedVariant = NAME_None;

		ILauncherDeviceGroupPtr ActiveGroup = DeviceGroup.Get();
		if (ActiveGroup.IsValid() && DeviceProxy.IsValid())
		{
			const TArray<FString>& DeviceIDs = ActiveGroup->GetDeviceIDs();
			for (int Index = 0; Index < DeviceIDs.Num(); ++Index)
			{
				const FString& DeviceID = DeviceIDs[Index];
				if (DeviceProxy->HasDeviceId(DeviceID))
				{
					SelectedVariant = DeviceProxy->GetTargetDeviceVariant(DeviceID);
					break;
				}
			}
		}


		SMultiColumnTableRow<TSharedPtr<ITargetDeviceProxy>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName - The name of the column to generate the widget for.
	 *
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "CheckBox")
		{
			return SAssignNew(DeviceCheckbox, SCheckBox)
				.IsChecked(this, &SProjectLauncherDeployTargetListRow::HandleCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SProjectLauncherDeployTargetListRow::HandleCheckBoxStateChanged)
				.ToolTipText(LOCTEXT("CheckBoxToolTip", "Check this box to include this device in the current device group"));
		}
		else if (ColumnName == "Device")
		{
			return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
						.WidthOverride(24.f)
						.HeightOverride(24.f)
						[
							SNew(SImage)
								.Image(this, &SProjectLauncherDeployTargetListRow::HandleDeviceImage)
						]
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleDeviceNameText)
				];
		}
		else if (ColumnName == "Variant")
		{
			if (DeviceProxy->CanSupportVariants())
			{
				return SNew(SBox)
					.Padding(FMargin(4.0, 0.0))
					.VAlign(VAlign_Center)
					[
						SNew(SProjectLauncherVariantSelector, DeviceProxy)
						.OnVariantSelected(this, &SProjectLauncherDeployTargetListRow::HandleVariantSelectorVariantSelected)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleVariantSelectorText)
					];
			}
			else
			{
				return SNew(SBox)
					.Padding(FMargin(4.0, 0.0))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleHostNoVariantText)
					];
			}
		}
		else if (ColumnName == "Platform")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleHostPlatformText)
				];
		}
		else if (ColumnName == "Host")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleHostNameText)
				];
		}
		else if (ColumnName == "Owner")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargetListRow::HandleHostUserText)
				];
		}

		return SNullWidget::NullWidget;
	}

private:

	// Callback for changing this row's check box state.
	void HandleCheckBoxStateChanged(ECheckBoxState NewState)
	{
		ILauncherDeviceGroupPtr ActiveGroup = DeviceGroup.Get();

		if (ActiveGroup.IsValid() && DeviceProxy.IsValid() && DeviceProxy->HasVariant(SelectedVariant))
		{
			const FString& DeviceID = DeviceProxy->GetTargetDeviceId(SelectedVariant);
			if (NewState == ECheckBoxState::Checked)
			{
				ActiveGroup->AddDevice(DeviceID);
			}
			else
			{
				ActiveGroup->RemoveDevice(DeviceID);
			}
		}
	}

	// Callback for determining this row's check box state.
	ECheckBoxState HandleCheckBoxIsChecked() const
	{
		if (IsEnabled())
		{
			ILauncherDeviceGroupPtr ActiveGroup = DeviceGroup.Get();

			if (ActiveGroup.IsValid() && DeviceProxy.IsValid() && DeviceProxy->HasVariant(SelectedVariant))
			{
				const FString& DeviceID = DeviceProxy->GetTargetDeviceId(SelectedVariant);
				return ActiveGroup->GetDeviceIDs().Contains(DeviceID) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}

		return ECheckBoxState::Unchecked;
	}

	// Callback for getting the text of the current variant
	FText HandleVariantSelectorText() const
	{
		ILauncherDeviceGroupPtr ActiveGroup = DeviceGroup.Get();
		if (ActiveGroup.IsValid() && DeviceProxy.IsValid())
		{
			if (SelectedVariant == NAME_None)
			{
				return LOCTEXT("DefaultVariant", "Default");
			}
			return FText::FromName(SelectedVariant);
		}

		return FText::GetEmpty();
	}

	// Callback for changing the selected variant
	void HandleVariantSelectorVariantSelected(FName InVariant)
	{
		if (DeviceProxy.IsValid() && DeviceProxy->HasVariant(SelectedVariant) && DeviceProxy->HasVariant(InVariant))
		{
			ILauncherDeviceGroupPtr ActiveGroup = DeviceGroup.Get();
			const FString& DeviceID = DeviceProxy->GetTargetDeviceId(SelectedVariant);
			if (ActiveGroup.IsValid() && ActiveGroup->GetDeviceIDs().Contains(DeviceID))
			{
				const FString& OldDeviceID = DeviceProxy->GetTargetDeviceId(SelectedVariant);
				const FString& NewDeviceID = DeviceProxy->GetTargetDeviceId(InVariant);
				
				if (ActiveGroup.IsValid())
				{
					ActiveGroup->RemoveDevice(OldDeviceID);
					ActiveGroup->AddDevice(NewDeviceID);
				}
			}

			SelectedVariant = InVariant;
		}
	}

	// Callback for getting the icon image of the device.
	const FSlateBrush* HandleDeviceImage() const
	{
		if (DeviceProxy->HasVariant(NAME_None))
		{
			const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(*DeviceProxy->GetTargetPlatformName(NAME_None));
			if (PlatformInfo)
			{
				FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal));
			}
		}
		return FStyleDefaults::GetNoBrush();
	}

	// Callback for getting the friendly name.
	FText HandleDeviceNameText() const
	{
		const FString& Name = DeviceProxy->GetName();

		if (Name.IsEmpty())
		{
			return LOCTEXT("UnnamedDeviceName", "<unnamed>");
		}

		return FText::FromString(Name);
	}

	// Callback for getting the host name.
	FText HandleHostNameText() const
	{
		return FText::FromString(DeviceProxy->GetHostName());
	}

	// Callback for getting the host user name.
	FText HandleHostUserText() const
	{
		return FText::FromString(DeviceProxy->GetHostUser());
	}

	// Callback for getting the host platform name.
	FText HandleHostPlatformText() const
	{
		if (DeviceProxy->HasVariant(NAME_None))
		{
			return FText::FromString(DeviceProxy->GetTargetPlatformName(NAME_None));
		}
		return LOCTEXT("InvalidVariant", "Invalid Variant");
	}

	// Callback for getting the default variant name in the case where 
	FText HandleHostNoVariantText() const
	{
		return LOCTEXT("StandardVariant", "Standard");
	}

private:

	// Holds a pointer to the device group that is being edited.
	TAttribute<ILauncherDeviceGroupPtr> DeviceGroup;

	// Holds a reference to the device proxy that is displayed in this row.
	TSharedPtr<ITargetDeviceProxy> DeviceProxy;

	TSharedPtr<SCheckBox> DeviceCheckbox;

	// Holds the name of the selected variant.
	FName SelectedVariant;	

	// Holds the highlight string for the log message.
	TAttribute<FText> HighlightText;
};


#undef LOCTEXT_NAMESPACE
