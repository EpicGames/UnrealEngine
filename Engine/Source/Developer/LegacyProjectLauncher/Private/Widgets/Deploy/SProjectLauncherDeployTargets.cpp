// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeployTargets.h"

#include "Styling/AppStyle.h"
#include "ITargetDeviceProxyManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"

#include "Widgets/Deploy/SProjectLauncherDeployTargetListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherDeployTargets"


/* SSessionSProjectLauncherDeployTargets structors
 *****************************************************************************/

SProjectLauncherDeployTargets::~SProjectLauncherDeployTargets()
{
	if (Model.IsValid())
	{
		const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = Model->GetDeviceProxyManager();
		DeviceProxyManager->OnProxyAdded().RemoveAll(this);
		DeviceProxyManager->OnProxyRemoved().RemoveAll(this);
	}
}


/* SSessionSProjectLauncherDeployTargets interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherDeployTargets::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel; 

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(10.0f, 10.0f)
			[			
				SAssignNew(PlatformComboBox, SComboBox<TSharedPtr<FName>>)			
					.ContentPadding(FMargin(6.0f, 2.0f))
					.OptionsSource(&VanillaPlatformList)
					.OnGenerateWidget(this, &SProjectLauncherDeployTargets::HandlePlatformComboBoxGenerateWidget)
					.OnSelectionChanged(this, &SProjectLauncherDeployTargets::HandlePlatformComboBoxSelectionChanged)
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherDeployTargets::HandlePlatformComboBoxContentText)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				// device list
				SAssignNew(DeviceProxyListView, SListView<TSharedPtr<ITargetDeviceProxy>>)
					.HeaderRow
					(
					SNew(SHeaderRow)

					+ SHeaderRow::Column("CheckBox")
						.DefaultLabel(LOCTEXT("DeviceListCheckboxColumnHeader", " "))
						.FixedWidth(24.0f)

					+ SHeaderRow::Column("Device")
						.DefaultLabel(LOCTEXT("DeviceListDeviceColumnHeader", "Device"))
						.FillWidth(0.35f)

					+ SHeaderRow::Column("Variant")
						.DefaultLabel(LOCTEXT("DeviceListVariantColumnHeader", "Variant"))
						.FillWidth(0.2f)

					+ SHeaderRow::Column("Platform")
						.DefaultLabel(LOCTEXT("DeviceListPlatformColumnHeader", "Platform"))
						.FillWidth(0.15f)

					+ SHeaderRow::Column("Host")
						.DefaultLabel(LOCTEXT("DeviceListHostColumnHeader", "Host"))
						.FillWidth(0.15f)

					+ SHeaderRow::Column("Owner")
						.DefaultLabel(LOCTEXT("DeviceListOwnerColumnHeader", "Owner"))
						.FillWidth(0.15f)
					)
				.ListItemsSource(&DeviceProxyList)
				.OnGenerateRow(this, &SProjectLauncherDeployTargets::HandleDeviceProxyListViewGenerateRow)
				.SelectionMode(ESelectionMode::Single)
				.Visibility(this, &SProjectLauncherDeployTargets::HandleDeviceProxyListViewVisibility)
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 12.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
					.Visibility(this, &SProjectLauncherDeployTargets::HandleNoDevicesBoxVisibility)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Icons.Warning")))
				]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(this, &SProjectLauncherDeployTargets::HandleNoDevicesTextBlockText)
					]
			]
	];

	const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = Model->GetDeviceProxyManager();

	DeviceProxyManager->OnProxyAdded().AddSP(this, &SProjectLauncherDeployTargets::HandleDeviceProxyManagerProxyAdded);
	DeviceProxyManager->OnProxyRemoved().AddSP(this, &SProjectLauncherDeployTargets::HandleDeviceProxyManagerProxyRemoved);
	DeviceProxyManager->GetProxies(NAME_None, false, DeviceProxyList);

	VanillaPlatformList.Reset();

	TSet<FName> VanillaNames;
	VanillaNames.Add(NAME_None);
	VanillaPlatformList.Add(MakeShareable(new FName(NAME_None)));

	for (FName VanillaName : PlatformInfo::GetAllVanillaPlatformNames())
	{
		if (!VanillaNames.Contains(VanillaName))
		{
			VanillaNames.Add(VanillaName);
			VanillaPlatformList.AddUnique(MakeShareable(new FName(VanillaName)));
		}
	}

	PlatformComboBox->SetSelectedItem(VanillaPlatformList[0]);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherDeployTargets implementation
*****************************************************************************/

void SProjectLauncherDeployTargets::RefreshDeviceProxyList()
{
	Model->GetDeviceProxyManager()->GetProxies(NAME_None, false, DeviceProxyList);
	DeviceProxyListView->RequestListRefresh();
}


/* SSessionSProjectLauncherDeployTargets callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

ILauncherDeviceGroupPtr SProjectLauncherDeployTargets::HandleDeviceListRowDeviceGroup() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->GetDeployedDeviceGroup();
	}

	return nullptr;
}


bool SProjectLauncherDeployTargets::HandleDeviceListRowIsEnabled(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const
{
	if (DeviceProxy.IsValid())
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			/*@Todo: Fix this! we should iterate the Devices target platforms and see if any are deployable*/
			//bool RetVal = SelectedProfile->IsDeployablePlatform(DeviceProxy->GetTargetPlatformName(NAME_None));
			return true;
		}
	}

	return false;
}


FText SProjectLauncherDeployTargets::HandleDeviceListRowToolTipText(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const
{
	FTextBuilder Builder;
	Builder.AppendLineFormat(LOCTEXT("DeviceListRowToolTipName", "Name: {0}"), FText::FromString(DeviceProxy->GetName()));
	
	if (DeviceProxy->HasVariant(NAME_None))
	{	
		Builder.AppendLineFormat(LOCTEXT("DeviceListRowToolTipPlatform", "Platform: {0}"), FText::FromString(DeviceProxy->GetTargetPlatformName(NAME_None)));
		Builder.AppendLineFormat(LOCTEXT("DeviceListRowToolTipDeviceId", "Device ID: {0}"), FText::FromString(DeviceProxy->GetTargetDeviceId(NAME_None)));
	}
	else
	{
		Builder.AppendLine(LOCTEXT("InvalidDevice", "Invalid Device"));
	}

	return Builder.ToText();
}


TSharedRef<ITableRow> SProjectLauncherDeployTargets::HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	check(Model->GetSelectedProfile().IsValid());

	return SNew(SProjectLauncherDeployTargetListRow, OwnerTable)
		.DeviceGroup(this, &SProjectLauncherDeployTargets::HandleDeviceListRowDeviceGroup)
		.DeviceProxy(InItem)
		.IsEnabled(this, &SProjectLauncherDeployTargets::HandleDeviceListRowIsEnabled, InItem)
		.ToolTipText(this, &SProjectLauncherDeployTargets::HandleDeviceListRowToolTipText, InItem);
}


EVisibility SProjectLauncherDeployTargets::HandleDeviceProxyListViewVisibility() const
{
	TSharedPtr<FName> DefaultPlatformNamePtr = PlatformComboBox->GetSelectedItem();
	const FName DefaultPlatformName = DefaultPlatformNamePtr.IsValid() ? *DefaultPlatformNamePtr.Get() : NAME_None;	

	if (DeviceProxyList.Num() > 0 && (DefaultPlatformName == NAME_None))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherDeployTargets::HandleNoDevicesBoxVisibility() const
{
	if (DeviceProxyList.Num() == 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherDeployTargets::HandleNoDevicesTextBlockText() const
{
	if (DeviceProxyList.Num() == 0)
	{
		return LOCTEXT("NoDevicesText", "No available devices were detected.");
	}

	return FText::GetEmpty();
}


void SProjectLauncherDeployTargets::HandleDeviceProxyManagerProxyAdded(const TSharedRef<ITargetDeviceProxy>& AddedProxy)
{
	RefreshDeviceProxyList();
}


void SProjectLauncherDeployTargets::HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& RemovedProxy)
{
	RefreshDeviceProxyList();
}


FText SProjectLauncherDeployTargets::HandlePlatformComboBoxContentText() const
{	
	FName DefaultPlatformName = NAME_None;
	if (Model.IsValid())
	{
		const ILauncherProfilePtr Profile = Model->GetSelectedProfile();
		if (Profile.IsValid())
		{
			DefaultPlatformName = Profile->GetDefaultDeployPlatform();
		}
	}

	FString ComboText(TEXT("Default Deploy Platform: "));
	ComboText += DefaultPlatformName.ToString();

	//avoid creating new FTexts as much as possible.
	if (DefaultPlatformText.ToString().Compare(ComboText) != 0)
	{
		DefaultPlatformText = FText::FromString(ComboText);
		for (int32 i = 0; i < VanillaPlatformList.Num(); ++i)
		{
			if (*VanillaPlatformList[i] == DefaultPlatformName)
			{
				PlatformComboBox->SetSelectedItem(VanillaPlatformList[i]);
				break;
			}
		}		
	}

	return DefaultPlatformText;
}


TSharedRef<SWidget> SProjectLauncherDeployTargets::HandlePlatformComboBoxGenerateWidget(TSharedPtr<FName> StringItem)
{
	FName DefaultPlatformName = StringItem.IsValid() ? *StringItem : NAME_None;	

	return SNew(STextBlock).Text(FText::FromName(DefaultPlatformName));
}


void SProjectLauncherDeployTargets::HandlePlatformComboBoxSelectionChanged(TSharedPtr<FName> StringItem, ESelectInfo::Type SelectInfo)
{
	if (Model.IsValid())
	{
		const ILauncherProfilePtr Profile = Model->GetSelectedProfile();
		if (Profile.IsValid())
		{
			Profile->SetDefaultDeployPlatform(StringItem.IsValid() ? *StringItem : NAME_None);
		}
	} 	
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
