// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SComboBox.h"

class ITargetDeviceProxy;


/**
 * Implements the deployment targets panel.
 */
class SProjectLauncherDeployTargets
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployTargets) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherDeployTargets( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/**
	 * Refreshes the list of device proxies.
	 */
	void RefreshDeviceProxyList( );

private:

	/** Callback for selecting a different device group in the device group selector. */
	void HandleDeviceGroupSelectorGroupSelected(const ILauncherDeviceGroupPtr& SelectedGroup);

	/** Callback for getting the currently selected device group for a device list row. */
	ILauncherDeviceGroupPtr HandleDeviceListRowDeviceGroup() const;

	/** Callback for getting the enabled state of a device proxy list row. */
	bool HandleDeviceListRowIsEnabled(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const;

	/** Callback for getting the tool tip text of a device proxy list row. */
	FText HandleDeviceListRowToolTipText(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const;

	/** Callback for generating a row in the device list view. */
	TSharedRef<ITableRow> HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Callback for determining the visibility of the device list view. */
	EVisibility HandleDeviceProxyListViewVisibility() const;

	/** Callback for getting a flag indicating whether a valid device group is currently selected. */
	EVisibility HandleDeviceSelectorVisibility() const;

	/** Callback for getting the visibility of the 'No devices detected' text block. */
	EVisibility HandleNoDevicesBoxVisibility() const;

	/** Callback for getting the text in the 'no devices' text block. */
	FText HandleNoDevicesTextBlockText() const;

	/** Callback for determining the visibility of the 'Select a device group' text block. */
	EVisibility HandleSelectDeviceGroupWarningVisibility() const;

	/** Callback for when a device proxy has been added to the device proxy manager. */
	void HandleDeviceProxyManagerProxyAdded(const TSharedRef<ITargetDeviceProxy>& AddedProxy);

	/** Callback for when a device proxy has been removed from the device proxy manager. */
	void HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& RemovedProxy);

	/** Callback for getting the name of the selected default platform. */
	FText HandlePlatformComboBoxContentText() const;

	/** Callback for generating widgets for the platforms combo box. */
	TSharedRef<SWidget> HandlePlatformComboBoxGenerateWidget(TSharedPtr<FName> StringItem);

	/** Callback for handling default platform selection changes. */
	void HandlePlatformComboBoxSelectionChanged(TSharedPtr<FName> StringItem, ESelectInfo::Type SelectInfo);

private:

	/** Holds the list of available device proxies. */
	TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxyList;

	/** Holds the list of vanilla platforms. */
	TArray<TSharedPtr<FName>> VanillaPlatformList;

	/** Holds the list of vanilla platform options. */
	TSharedPtr<SComboBox<TSharedPtr<FName>>> PlatformComboBox;
	mutable FText DefaultPlatformText;

	/** Holds the device proxy list view . */
	TSharedPtr<SListView<TSharedPtr<ITargetDeviceProxy>> > DeviceProxyListView;

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

};
