// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeployToDeviceSettings.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Deploy/SProjectLauncherDeployTargets.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherDeployToDeviceSettings"


/* SProjectLauncherDeployToDeviceSettings interface
 *****************************************************************************/

void SProjectLauncherDeployToDeviceSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, EVisibility InShowAdvanced)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.Padding(8.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						// deploy targets area
						SNew(SProjectLauncherDeployTargets, InModel)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SVerticalBox)
				.Visibility(InShowAdvanced)

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SExpandableArea)
							.AreaTitle(LOCTEXT("AdvancedAreaTitle", "Advanced Settings"))
							.InitiallyCollapsed(true)
							.Padding(8.0f)
							.BodyContent()
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									// incremental cook check box
									SNew(SCheckBox)
									.IsChecked(this, &SProjectLauncherDeployToDeviceSettings::HandleIncrementalCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherDeployToDeviceSettings::HandleIncrementalCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("IncrementalCheckBoxTooltip", "If checked, only modified content will be deployed, resulting in much faster deploy times. It is recommended to enable this option whenever possible."))
									.Content()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("IncrementalCheckBoxText", "Only deploy modified content"))
									]
								]
							]
					]
			]
	];
}


/* SProjectLauncherDeployToDeviceSettings callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherDeployToDeviceSettings::HandleIncrementalCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetIncrementalDeploying(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherDeployToDeviceSettings::HandleIncrementalCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsDeployingIncrementally())
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
