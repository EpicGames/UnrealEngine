// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

enum class ECheckBoxState : uint8;

/**
 * Implements the deploy-to-device settings panel.
 */
class SProjectLauncherDeployToDeviceSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeployToDeviceSettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The Slate argument list.
	 * @param InModel - The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, EVisibility InShowAdvanced = EVisibility::Visible );

protected:

private:

	/** Handles check state changes of the 'Incremental' check box. */
	void HandleIncrementalCheckBoxCheckStateChanged( ECheckBoxState NewState );

	/** Handles determining the checked state of the 'Incremental' check box. */
	ECheckBoxState HandleIncrementalCheckBoxIsChecked( ) const;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
