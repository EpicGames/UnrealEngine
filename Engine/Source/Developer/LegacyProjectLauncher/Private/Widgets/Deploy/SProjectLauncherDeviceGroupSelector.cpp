// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherDeviceGroupSelector.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Shared/SProjectLauncherFormLabel.h"
#include "Widgets/Input/SEditableComboBox.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherDeviceGroupSelector"



void SProjectLauncherDeviceGroupSelector::Construct(const FArguments& InArgs, const ILauncherProfileManagerRef& InProfileManager)
{
	OnGroupSelected = InArgs._OnGroupSelected;

	ProfileManager = InProfileManager;

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SProjectLauncherFormLabel)
				.LabelText(LOCTEXT("DeviceGroupComboBoxLabel", "Device group to deploy to:"))
			]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0, 4.0, 0.0, 0.0)
				[
					SAssignNew(DeviceGroupComboBox, SEditableComboBox<ILauncherDeviceGroupPtr>)
					.InitiallySelectedItem(InArgs._InitiallySelectedGroup)
					.OptionsSource(&ProfileManager->GetAllDeviceGroups())
					.AddButtonToolTip(LOCTEXT("AddProfileButtonToolTip", "Add a new device group"))
					.RemoveButtonToolTip(LOCTEXT("DeleteProfileButtonToolTip", "Delete the selected device group"))
					.RenameButtonToolTip(LOCTEXT("RenameProfileButtonToolTip", "Rename the selected device group"))
					.OnAddClicked(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxAddClicked)
					.OnGenerateWidget(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxGenerateWidget)
					.OnGetEditableText(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxGetEditableText)
					.OnRemoveClicked(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxRemoveClicked)
					.OnSelectionChanged(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxSelectionChanged)
					.OnSelectionRenamed(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxSelectionRenamed)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxContent)
					]
				]
		];

	ProfileManager->OnDeviceGroupAdded().AddSP(this, &SProjectLauncherDeviceGroupSelector::HandleProfileManagerDeviceGroupsChanged);
	ProfileManager->OnDeviceGroupRemoved().AddSP(this, &SProjectLauncherDeviceGroupSelector::HandleProfileManagerDeviceGroupsChanged);
}

FText SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxContent() const
{
	ILauncherDeviceGroupPtr SelectedGroup = DeviceGroupComboBox->GetSelectedItem();

	if (SelectedGroup.IsValid())
	{
		return FText::FromString(SelectedGroup->GetName());
	}

	return LOCTEXT("CreateOrSelectGroupText", "Create or select a device group...");
}

TSharedRef<SWidget> SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxGenerateWidget(ILauncherDeviceGroupPtr InItem)
{
	return SNew(STextBlock)
		.Text(this, &SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxWidgetText, InItem);
}

FString SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxGetEditableText()
{
	ILauncherDeviceGroupPtr SelectedGroup = DeviceGroupComboBox->GetSelectedItem();

	if (SelectedGroup.IsValid())
	{
		return SelectedGroup->GetName();
	}

	return FString();
}

FReply SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxRemoveClicked()
{
	ILauncherDeviceGroupPtr SelectedGroup = DeviceGroupComboBox->GetSelectedItem();

	if (SelectedGroup.IsValid())
	{
		ProfileManager->RemoveDeviceGroup(SelectedGroup.ToSharedRef());
	}

	if (ProfileManager->GetAllDeviceGroups().Num() > 0)
	{
		DeviceGroupComboBox->SetSelectedItem(ProfileManager->GetAllDeviceGroups()[0]);
	}
	else
	{
		DeviceGroupComboBox->SetSelectedItem(NULL);
	}

	return FReply::Handled();
}

FText SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxWidgetText(ILauncherDeviceGroupPtr Group) const
{
	if (Group.IsValid())
	{
		return FText::FromString(Group->GetName());
	}

	return FText::GetEmpty();
}

void SProjectLauncherDeviceGroupSelector::HandleProfileManagerDeviceGroupsChanged(const ILauncherDeviceGroupRef& /*ChangedProfile*/)
{
	DeviceGroupComboBox->RefreshOptions();
}

ILauncherDeviceGroupPtr SProjectLauncherDeviceGroupSelector::GetSelectedGroup() const
{
	return DeviceGroupComboBox->GetSelectedItem();
}

void SProjectLauncherDeviceGroupSelector::SetSelectedGroup(const ILauncherDeviceGroupPtr& DeviceGroup)
{
	if (!DeviceGroup.IsValid() || ProfileManager->GetAllDeviceGroups().Contains(DeviceGroup))
	{
		DeviceGroupComboBox->SetSelectedItem(DeviceGroup);
	}
}

FReply SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxAddClicked()
{
	ILauncherDeviceGroupPtr NewGroup = ProfileManager->AddNewDeviceGroup();

	DeviceGroupComboBox->SetSelectedItem(NewGroup);

	return FReply::Handled();
}

void SProjectLauncherDeviceGroupSelector::HandleDeviceGroupComboBoxSelectionRenamed(const FText& CommittedText, ETextCommit::Type)
{
	DeviceGroupComboBox->GetSelectedItem()->SetName(CommittedText.ToString());
}

#undef LOCTEXT_NAMESPACE
