// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileManager.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Delegate type for device group selection changes.
 *
 * The first parameter is the selected device group (or NULL if the previously selected group was unselected).
 */
DECLARE_DELEGATE_OneParam(FOnProjectLauncherDeviceGroupSelected, const ILauncherDeviceGroupPtr&)


/**
 * Implements a widget for device group selection.
 */
class SProjectLauncherDeviceGroupSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherDeviceGroupSelector) { }

		/**
		 * Exposes the initially selected device group.
		 */
		SLATE_ARGUMENT(ILauncherDeviceGroupPtr, InitiallySelectedGroup)
		
		/**
		 * Exposes a delegate to be invoked when a different device group has been selected.
		 */
		SLATE_EVENT(FOnProjectLauncherDeviceGroupSelected, OnGroupSelected)

	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherDeviceGroupSelector( )
	{
		if (ProfileManager.IsValid())
		{
			ProfileManager->OnDeviceGroupAdded().RemoveAll(this);
			ProfileManager->OnDeviceGroupRemoved().RemoveAll(this);
		}
	}

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The Slate argument list.
	 * @param InProfileManager - The profile manager to use.
	 */
	void Construct( const FArguments& InArgs, const ILauncherProfileManagerRef& InProfileManager );

	/**
	 * Gets the currently selected device group.
	 *
	 * @return The selected group, or NULL if no group is selected.
	 *
	 * @see SetSelectedGroup
	 */
	ILauncherDeviceGroupPtr GetSelectedGroup( ) const;

	/**
	 * Sets the selected device group.
	 *
	 * @param DeviceGroup - The device group to select.
	 *
	 * @see GetSelectedGroup
	 */
	void SetSelectedGroup( const ILauncherDeviceGroupPtr& DeviceGroup );

private:

	// Callback for clicking the 'Add device group' button
	FReply HandleDeviceGroupComboBoxAddClicked( );

	// Callback for getting the device group combo box content.
	FText HandleDeviceGroupComboBoxContent() const;

	// Callback for generating a row for the device group combo box.
	TSharedRef<SWidget> HandleDeviceGroupComboBoxGenerateWidget( ILauncherDeviceGroupPtr InItem );

	// Callback for getting the editable text of the currently selected device group.
	FString HandleDeviceGroupComboBoxGetEditableText( );

	// Callback for clicking the 'Delete device group' button
	FReply HandleDeviceGroupComboBoxRemoveClicked( );

	// Callback for changing the selected device group in the device group combo box.
	void HandleDeviceGroupComboBoxSelectionChanged( ILauncherDeviceGroupPtr Selection, ESelectInfo::Type SelectInfo )
	{
		OnGroupSelected.ExecuteIfBound(Selection);
	}

	// Callback for when the selected item in the device group combo box has been renamed.
	void HandleDeviceGroupComboBoxSelectionRenamed( const FText& CommittedText, ETextCommit::Type );

	// Callback for getting the text of a device group combo box widget.
	FText HandleDeviceGroupComboBoxWidgetText( ILauncherDeviceGroupPtr Group ) const;

	// Callback for changing the list of groups in the profile manager.
	void HandleProfileManagerDeviceGroupsChanged( const ILauncherDeviceGroupRef& /*ChangedProfile*/ );

private:

	// Holds the device group combo box.
	TSharedPtr<SEditableComboBox<ILauncherDeviceGroupPtr> > DeviceGroupComboBox;

	// Holds the profile manager.
	ILauncherProfileManagerPtr ProfileManager;

private:

	// Holds a delegate to be invoked when a different device group has been selected.
	FOnProjectLauncherDeviceGroupSelected OnGroupSelected;
};

