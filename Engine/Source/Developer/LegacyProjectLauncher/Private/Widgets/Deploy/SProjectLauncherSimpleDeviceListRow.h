// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Shared/ProjectLauncherDelegates.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Styling/AppStyle.h"
#include "PlatformInfo.h"
#include "Widgets/Shared/SProjectLauncherBuildConfigurationSelector.h"
#include "Widgets/Shared/SProjectLauncherCookModeSelector.h"
#include "Widgets/Shared/SProjectLauncherProfileLaunchButton.h"
#include "Widgets/Shared/SProjectLauncherVariantSelector.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherSimpleDeviceListRow"

/**
 * Implements a row widget for the launcher's device proxy list.
 */
class SProjectLauncherSimpleDeviceListRow
	: public STableRow<TSharedPtr<ITargetDeviceProxy>>
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherSimpleDeviceListRow) { }
		/**
		* The Callback for when the launch button is clicked.
		*/
		SLATE_EVENT(FOnProfileRun, OnProfileRun)
	
		/**
		 * The device proxy shown in this row.
		 */
		SLATE_ARGUMENT(TSharedPtr<ITargetDeviceProxy>, DeviceProxy)

		/**
		 * Whether the advanced options should be shown.
		 */
		SLATE_ATTRIBUTE(bool, IsAdvanced)

	SLATE_END_ARGS()

public:

	~SProjectLauncherSimpleDeviceListRow()
	{
		if (LaunchProfile.IsValid())
		{
			Model->GetProfileManager()->RemoveProfile(LaunchProfile.ToSharedRef());
		}
	}

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The construction arguments.
	 * @param InModel - The launcher model this list uses.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow< TSharedPtr<ITargetDeviceProxy> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(false),
			InOwnerTableView
			);

		Model = InModel;
		IsAdvanced = InArgs._IsAdvanced;
		OnProfileRun = InArgs._OnProfileRun;
		DeviceProxy = InArgs._DeviceProxy;

		SimpleProfile = Model->GetProfileManager()->FindOrAddSimpleProfile(DeviceProxy->GetName());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LaunchProfile = Model->GetProfileManager()->CreateUnsavedProfile(DeviceProxy->GetName());
		LaunchProfile->SetProjectSpecified(false); // device launch profiles should always use the fallback project & build target
		LaunchProfile->SetBuildTargetSpecified(false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		UpdateProfile();

		TSharedRef<SUniformGridPanel> NameGrid = SNew(SUniformGridPanel).SlotPadding(FMargin(0.0f, 1.0f));
		TSharedRef<SUniformGridPanel> ValueGrid = SNew(SUniformGridPanel).SlotPadding(FMargin(0.0f, 1.0f));
		
		MakeAdvancedSettings(NameGrid, ValueGrid);

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(0,0,0,1)
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(14, 0, 12, 0)
					[
						SNew(SBox)
						.WidthOverride(44.f)
						.HeightOverride(44.f)
						[
							SNew(SImage)
							.Image(this, &SProjectLauncherSimpleDeviceListRow::HandleDeviceImage)
						]
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2,9,2,4)
						[
							SNew(STextBlock)
							.Text(this, &SProjectLauncherSimpleDeviceListRow::HandleDeviceNameText)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2, 4, 2, 9)
						[
							SNew(STextBlock)
							.Text(this, &SProjectLauncherSimpleDeviceListRow::HandleHostPlatformText)
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						// This Vertical box ensures the NameGrid spans only the vertical space the ValueGrid forces.
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 0, 30, 0)
							[
								NameGrid
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								ValueGrid
							]
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(12, 5, 0, 5)
					[
						SNew(SSeparator)
						.Orientation(Orient_Vertical)
						.Thickness(1.f)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(20, 0, 20, 0)
					[
						SNew(SProjectLauncherProfileLaunchButton, true)
						.LaunchProfile(this, &SProjectLauncherSimpleDeviceListRow::GetLaunchProfile)
						.OnClicked(this, &SProjectLauncherSimpleDeviceListRow::OnLaunchClicked)
					]
				]
			]
		];

	}

private:

	// Create simple settings

	void MakeSimpleSettings(TSharedRef<SUniformGridPanel> InNameColumn, TSharedRef<SUniformGridPanel> InValueColumn)
	{
		int Row = 0;
		if (DeviceProxy->CanSupportVariants())
		{
			MakeVariantRow(InNameColumn, InValueColumn, Row++);
		}
	}

	// Create Advanced settings

	void MakeAdvancedSettings(TSharedRef<SUniformGridPanel> InNameColumn, TSharedRef<SUniformGridPanel> InValueColumn)
	{
		int Row = 0;
		if (DeviceProxy->GetNumVariants() > 1)
		{
			MakeVariantRow(InNameColumn, InValueColumn, Row++);
		}
		MakeBuildConfigurationRow(InNameColumn, InValueColumn, Row++);
		MakeCookModeRow(InNameColumn, InValueColumn, Row++);
	}

	// Type Settings Row

	void MakeVariantRow(TSharedRef<SUniformGridPanel> InNameColumn, TSharedRef<SUniformGridPanel> InValueColumn, int InRowIndex)
	{
		InNameColumn->AddSlot(0, InRowIndex)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LaunchVariantLabel", "Variant"))
			];

		InValueColumn->AddSlot(0, InRowIndex)
			.HAlign(HAlign_Fill)
			[
				SNew(SProjectLauncherVariantSelector, DeviceProxy)
				.OnVariantSelected(this, &SProjectLauncherSimpleDeviceListRow::HandleVariantSelectorVariantSelected)
				.Text(this, &SProjectLauncherSimpleDeviceListRow::HandleVariantSelectorText)
			];
	}

	FText HandleVariantSelectorText() const
	{
		if (SimpleProfile.IsValid())
		{
			FName Variant = SimpleProfile->GetDeviceVariant();
			if (Variant == NAME_None)
			{
				return LOCTEXT("DefaultVariant", "Default");
			}
			return FText::FromName(Variant);
		}

		return FText::GetEmpty();
	}

	void HandleVariantSelectorVariantSelected(FName InVariant)
	{
		if (SimpleProfile.IsValid())
		{
			SimpleProfile->SetDeviceVariant(InVariant);
			UpdateProfile();
		}
	}

	// Build Config Settings Row

	void MakeBuildConfigurationRow(TSharedRef<SUniformGridPanel> InNameColumn, TSharedRef<SUniformGridPanel> InValueColumn, int InRowIndex)
	{
		InNameColumn->AddSlot(0, InRowIndex)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LaunchConfigLabel", "Config"))
				.Visibility(this, &SProjectLauncherSimpleDeviceListRow::IsAdvancedVisible)
			];

		InValueColumn->AddSlot(0, InRowIndex)
			.HAlign(HAlign_Fill)
			[
				SNew(SProjectLauncherBuildConfigurationSelector)
				.OnConfigurationSelected(this, &SProjectLauncherSimpleDeviceListRow::HandleBuildConfigurationSelectorConfigurationSelected)
				.Text(this, &SProjectLauncherSimpleDeviceListRow::HandleBuildConfigurationSelectorText)
				.Visibility(this, &SProjectLauncherSimpleDeviceListRow::IsAdvancedVisible)
			];
	}

	FText HandleBuildConfigurationSelectorText() const
	{
		if (SimpleProfile.IsValid())
		{
			return EBuildConfigurations::ToText(SimpleProfile->GetBuildConfiguration());
		}

		return FText::GetEmpty();
	}

	void HandleBuildConfigurationSelectorConfigurationSelected(EBuildConfiguration Configuration)
	{
		if (SimpleProfile.IsValid())
		{
			SimpleProfile->SetBuildConfiguration(Configuration);
			UpdateProfile();
		}
	}

	// Cook Mode Settings Row

	void MakeCookModeRow(TSharedRef<SUniformGridPanel> InNameColumn, TSharedRef<SUniformGridPanel> InValueColumn, int InRowIndex)
	{
		InNameColumn->AddSlot(0, InRowIndex)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CookLabel", "Data Build"))
				.Visibility(this, &SProjectLauncherSimpleDeviceListRow::IsAdvancedVisible)
			];

		InValueColumn->AddSlot(0, InRowIndex)
			.HAlign(HAlign_Fill)
			[
				SNew(SProjectLauncherCookModeSelector)
				.OnCookModeSelected(this, &SProjectLauncherSimpleDeviceListRow::HandleCookModeSelectorConfigurationSelected)
				.Text(this, &SProjectLauncherSimpleDeviceListRow::HandleCookModeSelectorText)
				.Visibility(this, &SProjectLauncherSimpleDeviceListRow::IsAdvancedVisible)
			];
	}

	FText HandleCookModeSelectorText() const
	{
		if (SimpleProfile.IsValid())
		{
			switch (SimpleProfile->GetCookMode())
			{
			case ELauncherProfileCookModes::DoNotCook:
				return LOCTEXT("CookMode_DoNotCook", "Do not cook");

			case ELauncherProfileCookModes::ByTheBook:
				return LOCTEXT("CookMode_ByTheBook", "By the book");

			case ELauncherProfileCookModes::OnTheFly:
				return LOCTEXT("CookMode_OnTheFly", "On the fly");
			}
		}

		return FText::GetEmpty();
	}

	void HandleCookModeSelectorConfigurationSelected(ELauncherProfileCookModes::Type CookMode)
	{
		if (SimpleProfile.IsValid())
		{
			SimpleProfile->SetCookMode(CookMode);
			UpdateProfile();
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void UpdateProfile()
	{
		check(DeviceProxy.IsValid());

		// This detects a corrupt device, it doesn't even have data for it's default variant Remove the profile.
		// Entry will show invalid.
		if (!DeviceProxy->HasVariant(NAME_None))
		{
			Model->GetProfileManager()->RemoveProfile(LaunchProfile.ToSharedRef());
			LaunchProfile = nullptr;
		}
		
		if (LaunchProfile.IsValid())
		{
			FName Variant = SimpleProfile->GetDeviceVariant();
		
			// If the profile refers to a variant no longer supported by this device fall back to the default.
			if (!DeviceProxy->HasVariant(Variant))
			{
				Variant = NAME_None;
				SimpleProfile->SetDeviceVariant(Variant);
			}

			// Setup the Profile
			LaunchProfile->SetDeploymentMode(ELauncherProfileDeploymentModes::FileServer);

			ILauncherDeviceGroupRef NewGroup = Model->GetProfileManager()->CreateUnmanagedDeviceGroup();

			// multi-device targets (All_<platform>_devices_on_<host>) return a list of devices 
			const TSet<FString>& TargetDeviceIds = DeviceProxy->GetTargetDeviceIds(NAME_None);
			for (TSet<FString>::TConstIterator ItDeviceID(TargetDeviceIds); ItDeviceID; ++ItDeviceID)
			{
				NewGroup->AddDevice(*ItDeviceID);
			}
			LaunchProfile->SetDeployedDeviceGroup(NewGroup);

			LaunchProfile->ClearCookedPlatforms();
			LaunchProfile->AddCookedPlatform(DeviceProxy->GetTargetPlatformName(Variant));

			bool Advanced = IsAdvanced.IsBound() && IsAdvanced.Get();
			if (Advanced)
			{
				LaunchProfile->SetBuildConfiguration(SimpleProfile->GetBuildConfiguration());
				LaunchProfile->SetCookMode(SimpleProfile->GetCookMode());
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ILauncherProfilePtr GetLaunchProfile() const
	{
		return LaunchProfile;
	}

	FReply OnLaunchClicked()
	{
		if (OnProfileRun.IsBound())
		{
			OnProfileRun.Execute(LaunchProfile.ToSharedRef());
		}

		return FReply::Handled();
	}

	// Callback for whether advanced options are shown.
	EVisibility IsAdvancedVisible() const
	{
		bool Advanced = IsAdvanced.IsBound() && IsAdvanced.Get();
		return Advanced ? EVisibility::Visible : EVisibility::Collapsed;
	}

	// Callback for getting the icon image of the device.
	const FSlateBrush* HandleDeviceImage( ) const
	{
		if (LaunchProfile.IsValid())
		{
			const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(*DeviceProxy->GetTargetPlatformName(SimpleProfile->GetDeviceVariant()));
			if (PlatformInfo)
			{
				return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Large));
			}
		}
		return FStyleDefaults::GetNoBrush();
	}

	// Callback for getting the friendly name.
	FText HandleDeviceNameText() const
	{
		const FString& Name = DeviceProxy->GetName();

		if (Name.IsEmpty())
		{
			return LOCTEXT("UnnamedDeviceName", "<unnamed>");
		}

		return FText::FromString(Name);
	}

	// Callback for getting the host name.
	FText HandleHostNameText( ) const
	{
		return FText::FromString(DeviceProxy->GetHostName());
	}

	// Callback for getting the host user name.
	FText HandleHostUserText( ) const
	{
		return FText::FromString(DeviceProxy->GetHostUser());
	}

	// Callback for getting the host platform name.
	FText HandleHostPlatformText( ) const
	{
		if (LaunchProfile.IsValid())
		{
			return FText::FromString(DeviceProxy->GetTargetPlatformName(SimpleProfile->GetDeviceVariant()));
		}
		return LOCTEXT("InvalidVariant", "Invalid Variant");
	}

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

	// Holds a reference to the device proxy that is displayed in this row.
	TSharedPtr<ITargetDeviceProxy> DeviceProxy;

	// Holds a reference to the simple profile for this device
	ILauncherSimpleProfilePtr SimpleProfile;

	// Holds a reference to the generated full launch profile for this device.
	ILauncherProfilePtr LaunchProfile;

	// Specifies that whether options are shown.
	TAttribute<bool> IsAdvanced;

	// Holds a delegate to be invoked when a profile is run.
	FOnProfileRun OnProfileRun;
};


#undef LOCTEXT_NAMESPACE
