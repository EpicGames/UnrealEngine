// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Deploy/SProjectLauncherSimpleDeviceListView.h"

#include "Framework/Docking/TabManager.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Deploy/SProjectLauncherSimpleDeviceListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherSimpleDeviceListView"


/* SProjectLauncherDeployTargets structors
 *****************************************************************************/

SProjectLauncherSimpleDeviceListView::~SProjectLauncherSimpleDeviceListView()
{
	if (Model.IsValid())
	{
		const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = Model->GetDeviceProxyManager();
		DeviceProxyManager->OnProxyAdded().RemoveAll(this);
		DeviceProxyManager->OnProxyRemoved().RemoveAll(this);
	}
}


/* SProjectLauncherDeployTargets interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherSimpleDeviceListView::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	OnProfileRun = InArgs._OnProfileRun;
	IsAdvanced = InArgs._IsAdvanced;
	
	Model = InModel;

	SAssignNew(DeviceProxyListView, SListView<TSharedPtr<ITargetDeviceProxy>>)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&DeviceProxyList)
		.OnGenerateRow(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceProxyListViewGenerateRow);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0)
		[
			SNew(SScrollBorder, DeviceProxyListView.ToSharedRef())
			[
				DeviceProxyListView.ToSharedRef()
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(0, 6))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(14, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ProjectLauncherDeviceManagerLinkPreamble", "Don't see your device? Verify it's setup and claimed in the "))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(0.0f, 0.0f)
				[
					// all cultures hyper link
					SNew(SHyperlink)
					.OnNavigate(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceManagerHyperlinkNavigate)
					.Text(LOCTEXT("ProjectLauncherDeviceManagerLink", "Device Manager."))
					.ToolTipText(LOCTEXT("ProjectLauncherDeviceManagerLinkTooltip", "Open the Device Manager window, where you can setup and claim devices connected to your machine or shared on the network."))
				]
			]
			
		]
	];

	const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = Model->GetDeviceProxyManager();

	DeviceProxyManager->OnProxyAdded().AddSP(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceProxyManagerProxyAdded);
	DeviceProxyManager->OnProxyRemoved().AddSP(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceProxyManagerProxyRemoved);

	// the list should also contain the aggregate (All_<platform>_devices_on_<host>) proxy
	DeviceProxyManager->GetAllProxies(NAME_None, DeviceProxyList);

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SProjectLauncherDeployTargets implementation
 *****************************************************************************/

void SProjectLauncherSimpleDeviceListView::RefreshDeviceProxyList()
{
	// the list should also contain the aggregate (All_<platform>_devices_on_<host>) proxy
	Model->GetDeviceProxyManager()->GetAllProxies(NAME_None, DeviceProxyList);
	DeviceProxyListView->RequestListRefresh();
}


/* SProjectLauncherDeployTargets callbacks
 *****************************************************************************/

bool SProjectLauncherSimpleDeviceListView::HandleDeviceListRowIsEnabled(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const
{
	return true;
}


void SProjectLauncherSimpleDeviceListView::HandleDeviceManagerHyperlinkNavigate() const
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId("DeviceManager"));
}


FText SProjectLauncherSimpleDeviceListView::HandleDeviceListRowToolTipText(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const
{
	FTextBuilder Builder;
	// aggregate (All_<platform>_devices_on_<host>) proxy - create the tool tip with the device list
	if (DeviceProxy->IsAggregated())
	{
		FString DeviceListStr;
		ILauncherSimpleProfilePtr SimpleProfile = Model->GetProfileManager()->FindSimpleProfile(DeviceProxy->GetName());
		FName ProfileName = SimpleProfile.IsValid()? SimpleProfile->GetDeviceVariant(): NAME_None;

		const TSet<FString>& TargetDeviceIds = DeviceProxy->GetTargetDeviceIds(ProfileName);
		for (TSet<FString>::TConstIterator ItDeviceId(TargetDeviceIds); ItDeviceId; ++ItDeviceId)
		{
			TSharedPtr<ITargetDeviceProxy> PhysicalDeviceProxy = Model->GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(*ItDeviceId);

			if (PhysicalDeviceProxy.IsValid())
			{
				DeviceListStr.AppendChar('\n');
				DeviceListStr.Append(*PhysicalDeviceProxy->GetName());
			}
		}
		Builder.AppendLineFormat(LOCTEXT("DeviceListAggregatedRowToolTipName", "Name: {0}\nDevices: {1}"), FText::FromString(DeviceProxy->GetName()), FText::FromString(DeviceListStr));
	}
	else
	{
		Builder.AppendLineFormat(LOCTEXT("DeviceListRowToolTipName", "Name: {0}"), FText::FromString(DeviceProxy->GetName()));
	}

	return Builder.ToText();
}


TSharedRef<ITableRow> SProjectLauncherSimpleDeviceListView::HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SProjectLauncherSimpleDeviceListRow, Model.ToSharedRef(), OwnerTable)
		.OnProfileRun(OnProfileRun)
		.IsAdvanced(IsAdvanced)
		.DeviceProxy(InItem)
		.IsEnabled(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceListRowIsEnabled, InItem)
		.ToolTipText(this, &SProjectLauncherSimpleDeviceListView::HandleDeviceListRowToolTipText, InItem);
}


void SProjectLauncherSimpleDeviceListView::HandleDeviceProxyManagerProxyAdded(const TSharedRef<ITargetDeviceProxy>& AddedProxy)
{
	RefreshDeviceProxyList();
}


void SProjectLauncherSimpleDeviceListView::HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& AddedProxy)
{
	RefreshDeviceProxyList();
}


#undef LOCTEXT_NAMESPACE
