// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Shared/ProjectLauncherDelegates.h"

class FProjectLauncherModel;
class ITableRow;
class ITargetDeviceProxy;
class STableViewBase;


/**
 * Implements the deployment targets panel.
 */
class SProjectLauncherSimpleDeviceListView
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherSimpleDeviceListView) { }
		SLATE_EVENT(FOnProfileRun, OnProfileRun)
		SLATE_ATTRIBUTE(bool, IsAdvanced)
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SProjectLauncherSimpleDeviceListView();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 * @param InAdvanced Whether or not the elements should show advanced data.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/** Refresh the list of device proxies. */
	void RefreshDeviceProxyList();

private:

	/** Callback for getting the enabled state of a device proxy list row. */
	bool HandleDeviceListRowIsEnabled(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const;

	/** Callback when the user clicks the Device Manager hyperlink. */
	void HandleDeviceManagerHyperlinkNavigate() const;

	/** Callback for getting the tool tip text of a device proxy list row. */
	FText HandleDeviceListRowToolTipText(TSharedPtr<ITargetDeviceProxy> DeviceProxy) const;

	/** Callback for generating a row in the device list view. */
	TSharedRef<ITableRow> HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Callback for when a device proxy has been added to the device proxy manager. */
	void HandleDeviceProxyManagerProxyAdded(const TSharedRef<ITargetDeviceProxy>& AddedProxy);

	/** Callback for when a device proxy has been added to the device proxy manager. */
	void HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& RemovedProxy);

private:

	/** Holds the list of available device proxies. */
	TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxyList;

	/** Holds the device proxy list view . */
	TSharedPtr<SListView<TSharedPtr<ITargetDeviceProxy>> > DeviceProxyListView;

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** Specifies whether advanced options are shown. */
	TAttribute<bool> IsAdvanced;

	/** Holds a delegate to be invoked when a profile is run. */
	FOnProfileRun OnProfileRun;
};
