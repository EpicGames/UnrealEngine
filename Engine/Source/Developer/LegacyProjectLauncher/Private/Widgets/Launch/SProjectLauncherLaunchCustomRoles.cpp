// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Launch/SProjectLauncherLaunchCustomRoles.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherLaunchCustomRoles"


/* SProjectLauncherLaunchCustomRoles structors
 *****************************************************************************/

SProjectLauncherLaunchCustomRoles::~SProjectLauncherLaunchCustomRoles()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherLaunchCustomRoles interface
 *****************************************************************************/

void SProjectLauncherLaunchCustomRoles::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNullWidget::NullWidget
	];
}


#undef LOCTEXT_NAMESPACE
