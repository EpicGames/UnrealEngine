// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

/**
 * Implements the settings panel for launching with custom roles.
 */
class SProjectLauncherLaunchCustomRoles
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherLaunchCustomRoles) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherLaunchCustomRoles( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The Slate argument list.
	 * @param InModel - The data model.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
