// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Launch/SProjectLauncherLaunchPage.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Launch/SProjectLauncherLaunchRoleEditor.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherLaunchPage"


/* SProjectLauncherLaunchPage structors
 *****************************************************************************/

SProjectLauncherLaunchPage::~SProjectLauncherLaunchPage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherLaunchPage interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherLaunchPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	// create launch modes menu
	FMenuBuilder LaunchModeMenuBuilder(true, NULL);
	{
		FUIAction DefaultRoleAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchPage::HandleLaunchModeMenuEntryClicked, ELauncherProfileLaunchModes::DefaultRole));
		LaunchModeMenuBuilder.AddMenuEntry(LOCTEXT("DefaultRoleAction", "Using default role"), LOCTEXT("DefaultRoleActionHint", "Launch with the default role on all deployed devices."), FSlateIcon(), DefaultRoleAction);
		
		FUIAction CustomRolesAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchPage::HandleLaunchModeMenuEntryClicked, ELauncherProfileLaunchModes::CustomRoles));
		LaunchModeMenuBuilder.AddMenuEntry(LOCTEXT("CustomRolesAction", "Using custom roles"), LOCTEXT("CustomRolesActionHint", "Launch with per-device custom roles."), FSlateIcon(), CustomRolesAction);

		FUIAction DoNotLaunchAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchPage::HandleLaunchModeMenuEntryClicked, ELauncherProfileLaunchModes::DoNotLaunch));
		LaunchModeMenuBuilder.AddMenuEntry(LOCTEXT("DoNotCookAction", "Do not launch"), LOCTEXT("DoNotCookActionHint", "Do not launch the build at this time."), FSlateIcon(), DoNotLaunchAction);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
					.Visibility(this, &SProjectLauncherLaunchPage::HandleLaunchModeBoxVisibility)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("HowToLaunchText", "How would you like to launch?"))
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0, 0.0, 0.0, 0.0)
					[
						// launch mode menu
						SNew(SComboButton)
							.ButtonContent()
							[
								SNew(STextBlock)
									.Text(this, &SProjectLauncherLaunchPage::HandleLaunchModeComboButtonContentText)
							]
							.ContentPadding(FMargin(6.0, 2.0))
							.MenuContent()
							[
								LaunchModeMenuBuilder.MakeWidget()
							]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(8.0f)
					.Visibility(this, &SProjectLauncherLaunchPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::CustomRolesNotSupportedYet)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SImage)
									.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("CopyToDeviceRequiresCookByTheBookText", "Custom launch roles are not supported yet."))
							]
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("DefaultRoleAreaTitle", "Default Role"))
					.InitiallyCollapsed(false)
					.Padding(8.0f)
					.Visibility(this, &SProjectLauncherLaunchPage::HandleLaunchSettingsVisibility)
					.BodyContent()
					[
						// launch settings area
						SAssignNew(DefaultRoleEditor, SProjectLauncherLaunchRoleEditor)
							.AvailableCultures(&AvailableCultures)
							.AvailableMaps(&AvailableMaps)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 4.0, 0.0, 0.0)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("CannotLaunchText", "The build is not being deployed and cannot be launched."))
					.Visibility(this, &SProjectLauncherLaunchPage::HandleCannotLaunchTextBlockVisibility)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherLaunchPage::HandleProfileManagerProfileSelected);

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->OnProjectChanged().AddSP(this, &SProjectLauncherLaunchPage::HandleProfileProjectChanged);
	}

	Refresh();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherLaunchPage::Refresh()
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == ELauncherProfileCookModes::ByTheBook)
		{
			AvailableCultures = SelectedProfile->GetCookedCultures();
		}
		else
		{
			FInternationalization::Get().GetCultureNames(AvailableCultures);
		}

		AvailableMaps = FGameProjectHelper::GetAvailableMaps(SelectedProfile->GetProjectBasePath(), SelectedProfile->SupportsEngineMaps(), true);

		DefaultRoleEditor->Refresh(SelectedProfile->GetDefaultLaunchRole());
	}
	else
	{
		AvailableCultures.Reset();
		AvailableMaps.Reset();

		DefaultRoleEditor->Refresh(NULL);
	}
}


/* SProjectLauncherLaunchPage callbacks
 *****************************************************************************/

EVisibility SProjectLauncherLaunchPage::HandleCannotLaunchTextBlockVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() == ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherLaunchPage::HandleLaunchModeBoxVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherLaunchPage::HandleLaunchModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfileLaunchModes::Type LaunchMode = SelectedProfile->GetLaunchMode();

		if (LaunchMode == ELauncherProfileLaunchModes::CustomRoles)
		{
			return LOCTEXT("LaunchModeComboButtonCustomRolesText", "Using custom roles");
		}

		if (LaunchMode == ELauncherProfileLaunchModes::DefaultRole)
		{
			return LOCTEXT("LaunchModeComboButtonDefaultRoleText", "Using default role");
		}

		if (LaunchMode == ELauncherProfileLaunchModes::DoNotLaunch)
		{
			return LOCTEXT("LaunchModeComboButtonDoNotLaunchText", "Do not launch");
		}

		return LOCTEXT("LaunchModeComboButtonSelectText", "Select...");
	}

	return FText::GetEmpty();
}


void SProjectLauncherLaunchPage::HandleLaunchModeMenuEntryClicked(ELauncherProfileLaunchModes::Type LaunchMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetLaunchMode(LaunchMode);
	}
}


EVisibility SProjectLauncherLaunchPage::HandleLaunchSettingsVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			if (SelectedProfile->GetLaunchMode() == ELauncherProfileLaunchModes::DefaultRole)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherLaunchPage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	if (PreviousProfile.IsValid())
	{
		PreviousProfile->OnProjectChanged().RemoveAll(this);
	}
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->OnProjectChanged().AddSP(this, &SProjectLauncherLaunchPage::HandleProfileProjectChanged);
	}
	Refresh();
}


void SProjectLauncherLaunchPage::HandleProfileProjectChanged()
{
	Refresh();
}


EVisibility SProjectLauncherLaunchPage::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
