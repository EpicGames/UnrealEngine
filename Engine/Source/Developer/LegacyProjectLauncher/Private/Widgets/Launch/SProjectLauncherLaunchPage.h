// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class Error;
class SProjectLauncherLaunchRoleEditor;

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherLaunchPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherLaunchPage) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherLaunchPage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

	/**
	 * Refreshes the widget.
	 */
	void Refresh( );

private:

	// Callback for getting the visibility of the 'Cannot launch' text block.
	EVisibility HandleCannotLaunchTextBlockVisibility( ) const;

	// Callback for determining the visibility of the 'Launch Mode' box.
	EVisibility HandleLaunchModeBoxVisibility( ) const;

	// Callback for getting the content text of the 'Launch Mode' combo button.
	FText HandleLaunchModeComboButtonContentText( ) const;

	// Callback for clicking an item in the 'Launch Mode' menu.
	void HandleLaunchModeMenuEntryClicked( ELauncherProfileLaunchModes::Type LaunchMode );

	// Callback for getting the visibility state of the launch settings area.
	EVisibility HandleLaunchSettingsVisibility( ) const;

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

	// Callback for determining the visibility of a validation error icon.
	EVisibility HandleValidationErrorIconVisibility( ELauncherProfileValidationErrors::Type Error ) const;

	// Callback for updating any settings after the selected project has changed in the profile
	void HandleProfileProjectChanged();

private:

	// Holds the default role editor.
	TSharedPtr<SProjectLauncherLaunchRoleEditor> DefaultRoleEditor;

	// Holds the list of cultures that are available for the selected game.
	TArray<FString> AvailableCultures;

	// Holds the list of maps that are available for the selected game.
	TArray<FString> AvailableMaps;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
