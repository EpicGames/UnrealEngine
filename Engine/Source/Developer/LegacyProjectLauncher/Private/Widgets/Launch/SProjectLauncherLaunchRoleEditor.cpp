// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Launch/SProjectLauncherLaunchRoleEditor.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Shared/SProjectLauncherFormLabel.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherLaunchRoleEditor"




BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherLaunchRoleEditor::Construct(const FArguments& InArgs)
{
	AvailableCultures = InArgs._AvailableCultures;
	AvailableMaps = InArgs._AvailableMaps;
	Role = InArgs._InitialRole;

	// create instance types menu
	FMenuBuilder InstanceTypeMenuBuilder(true, NULL);
	{
		FUIAction StandaloneClientAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchRoleEditor::HandleInstanceTypeMenuEntryClicked, ELauncherProfileRoleInstanceTypes::StandaloneClient));
		InstanceTypeMenuBuilder.AddMenuEntry(LOCTEXT("StandaloneClient", "Standalone Client"), LOCTEXT("StandaloneClientActionHint", "Launch this instance as a standalone game client."), FSlateIcon(), StandaloneClientAction);

		FUIAction ListenServerAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchRoleEditor::HandleInstanceTypeMenuEntryClicked, ELauncherProfileRoleInstanceTypes::ListenServer));
		InstanceTypeMenuBuilder.AddMenuEntry(LOCTEXT("ListenServer", "Listen Server"), LOCTEXT("ListenServerActionHint", "Launch this instance as a game client that can accept connections from other clients."), FSlateIcon(), ListenServerAction);

		FUIAction DedicatedServerAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchRoleEditor::HandleInstanceTypeMenuEntryClicked, ELauncherProfileRoleInstanceTypes::DedicatedServer));
		InstanceTypeMenuBuilder.AddMenuEntry(LOCTEXT("DedicatedServer", "Dedicated Server"), LOCTEXT("DedicatedServerActionHint", "Launch this instance as a dedicated game server."), FSlateIcon(), DedicatedServerAction);

		FUIAction UnrealEditorAction(FExecuteAction::CreateSP(this, &SProjectLauncherLaunchRoleEditor::HandleInstanceTypeMenuEntryClicked, ELauncherProfileRoleInstanceTypes::UnrealEditor));
		InstanceTypeMenuBuilder.AddMenuEntry(LOCTEXT("UnrealEditor", "Unreal Editor"), LOCTEXT("UnrealEditorActionHint", "Launch this instance as an Unreal Editor."), FSlateIcon(), UnrealEditorAction);
	}

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SProjectLauncherFormLabel)
				.LabelText(LOCTEXT("InstanceTypeComboBoxLabel", "Launch As:"))
			]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					// instance type menu
					SNew(SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherLaunchRoleEditor::HandleInstanceTypeComboButtonContentText)
					]
					.ContentPadding(FMargin(6.0f, 2.0f))
						.MenuContent()
						[
							InstanceTypeMenuBuilder.MakeWidget()
						]
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f, 0.0f, 0.0f)
				[
					SNew(SProjectLauncherFormLabel)
					.ErrorToolTipText(LOCTEXT("CultureNotAvailableError", "The selected culture is not being cooked or is not available."))
					.ErrorVisibility(this, &SProjectLauncherLaunchRoleEditor::HandleCultureValidationErrorIconVisibility)
					.LabelText(LOCTEXT("InitialCultureTextBoxLabel", "Initial Culture:"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f, 0.0f, 0.0f)
				[
					// initial culture combo box
					SAssignNew(CultureComboBox, STextComboBox)
					.ColorAndOpacity(this, &SProjectLauncherLaunchRoleEditor::HandleCultureComboBoxColorAndOpacity)
					.OptionsSource(&CultureList)
					.OnSelectionChanged(this, &SProjectLauncherLaunchRoleEditor::HandleCultureComboBoxSelectionChanged)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f, 0.0f, 0.0f)
				[
					SNew(SProjectLauncherFormLabel)
					.ErrorToolTipText(LOCTEXT("MapNotAvailableError", "The selected map is not being cooked or is not available."))
					.ErrorVisibility(this, &SProjectLauncherLaunchRoleEditor::HandleMapValidationErrorIconVisibility)
					.LabelText(LOCTEXT("InitialMapTextBoxLabel", "Initial Map:"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f, 0.0f, 0.0f)
				[
					// initial map combo box
					SAssignNew(MapComboBox, STextComboBox)
					.ColorAndOpacity(this, &SProjectLauncherLaunchRoleEditor::HandleMapComboBoxColorAndOpacity)
					.OptionsSource(&MapList)
					.OnSelectionChanged(this, &SProjectLauncherLaunchRoleEditor::HandleMapComboBoxSelectionChanged)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 8.0f, 0.0f, 0.0f)
				[
					SNew(SProjectLauncherFormLabel)
					.LabelText(LOCTEXT("CommandLineTextBoxLabel", "Additional Command Line Parameters:"))
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f, 0.0f, 0.0f)
				[
					// command line text box
					SAssignNew(CommandLineTextBox, SEditableTextBox)
					.OnTextChanged(this, &SProjectLauncherLaunchRoleEditor::HandleCommandLineTextBoxTextChanged)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 12.0f, 0.0f, 0.0f)
				[
					// v-sync check box
					SNew(SCheckBox)
					.IsChecked(this, &SProjectLauncherLaunchRoleEditor::HandleVsyncCheckBoxIsChecked)
					.OnCheckStateChanged(this, &SProjectLauncherLaunchRoleEditor::HandleVsyncCheckBoxCheckStateChanged)
					.Padding(FMargin(4.0f, 0.0f))
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("VsyncCheckBoxText", "Synchronize Screen Refresh Rate (VSync)"))
					]
				]
		];

	Refresh(InArgs._InitialRole);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SProjectLauncherLaunchRoleEditor::Refresh(const ILauncherProfileLaunchRolePtr& InRole)
{
	Role = InRole;

	ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

	CultureList.Reset();
	CultureList.Add(MakeShareable(new FString(LOCTEXT("DefaultCultureText", "<default>").ToString())));

	MapList.Reset();
	MapList.Add(MakeShareable(new FString(LOCTEXT("DefaultMapText", "<default>").ToString())));

	if (RolePtr.IsValid())
	{
		// refresh widgets
		CommandLineTextBox->SetText(FText::FromString(*RolePtr->GetUATCommandLine()));

		// rebuild culture list
		if (AvailableCultures != NULL)
		{
			for (int32 AvailableCultureIndex = 0; AvailableCultureIndex < AvailableCultures->Num(); ++AvailableCultureIndex)
			{
				TSharedPtr<FString> Culture = MakeShareable(new FString((*AvailableCultures)[AvailableCultureIndex]));

				CultureList.Add(Culture);

				if (*Culture == RolePtr->GetInitialCulture())
				{
					CultureComboBox->SetSelectedItem(Culture);
				}
			}
		}

		if (RolePtr->GetInitialCulture().IsEmpty() || (CultureList.Num() == 0))
		{
			CultureComboBox->SetSelectedItem(CultureList[0]);
		}

		// rebuild map list
		if (AvailableMaps != NULL)
		{
			for (int32 AvailableMapIndex = 0; AvailableMapIndex < AvailableMaps->Num(); ++AvailableMapIndex)
			{
				TSharedPtr<FString> Map = MakeShareable(new FString((*AvailableMaps)[AvailableMapIndex]));

				MapList.Add(Map);

				if (*Map == RolePtr->GetInitialMap())
				{
					MapComboBox->SetSelectedItem(Map);
				}
			}
		}

		if (RolePtr->GetInitialMap().IsEmpty() || (MapList.Num() == 0))
		{
			MapComboBox->SetSelectedItem(MapList[0]);
		}
	}
	else
	{
		CommandLineTextBox->SetText(FText::GetEmpty());
		CultureComboBox->ClearSelection();
		MapComboBox->ClearSelection();
	}

	CultureComboBox->RefreshOptions();
	MapComboBox->RefreshOptions();
}



#undef LOCTEXT_NAMESPACE
