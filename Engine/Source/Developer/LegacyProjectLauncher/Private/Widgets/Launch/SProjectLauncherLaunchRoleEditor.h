// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileLaunchRole.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SComboBox.h"

class ILauncherProfile;
class SEditableTextBox;

/**
 * Implements the settings panel for a single launch role.
 */
class SProjectLauncherLaunchRoleEditor
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherLaunchRoleEditor)
		: _AvailableCultures()
		, _AvailableMaps()
	{ }
		
		/** The role to be edited initially. */
		SLATE_ARGUMENT(ILauncherProfileLaunchRolePtr, InitialRole)

		/** The list of available cultures. */
		SLATE_ARGUMENT(const TArray<FString>*, AvailableCultures)

		/** The list of available maps. */
		SLATE_ARGUMENT(const TArray<FString>*, AvailableMaps)

	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InRole The launch role to edit.
	 */
	void Construct(	const FArguments& InArgs );

	/**
	 * Refreshes the widget.
	 *
	 * @param InRole The role to edit, or NULL if no role is being edited.
	 */
	void Refresh( const ILauncherProfileLaunchRolePtr& InRole );

private:

	// Callback for when the text of the command line text box has changed.
	void HandleCommandLineTextBoxTextChanged( const FText& InText )
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			RolePtr->SetCommandLine(InText.ToString());
		}
	}

	// Callback for getting the name of the selected instance type.
	FText HandleInstanceTypeComboButtonContentText( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			return FText::FromString(ELauncherProfileRoleInstanceTypes::ToString(RolePtr->GetInstanceType()));
		}

		return FText::GetEmpty();
	}

	// Callback for clicking an item in the 'Instance type' menu.
	void HandleInstanceTypeMenuEntryClicked( ELauncherProfileRoleInstanceTypes::Type InstanceType )
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			RolePtr->SetInstanceType(InstanceType);
		}
	}

	// Callback for getting the foreground text color of the culture combo box.
	FSlateColor HandleCultureComboBoxColorAndOpacity( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			const FString& InitialCulture = RolePtr->GetInitialCulture();

			if (InitialCulture.IsEmpty() || (AvailableCultures->Contains(InitialCulture)))
			{
				return FSlateColor::UseForeground();
			}
		}

		return FLinearColor::Red;
	}

	// Callback for changing the initial culture selection.
	void HandleCultureComboBoxSelectionChanged( TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo )
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			if (Selection.IsValid() && (Selection != CultureList[0]))
			{
				RolePtr->SetInitialCulture(*Selection);
			}
			else
			{
				RolePtr->SetInitialCulture(FString());
			}
		}
	}

	// Callback for getting the visibility of the validation error icon of the 'Initial Culture' combo box.
	EVisibility HandleCultureValidationErrorIconVisibility( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			const FString& InitialCulture = RolePtr->GetInitialCulture();

			if (InitialCulture.IsEmpty() || (AvailableCultures->Contains(InitialCulture)))
			{
				return EVisibility::Hidden;
			}
		}

		return EVisibility::Visible;
	}

	// Callback for getting the visibility of the validation error icon of the 'Launch As' combo box.
	EVisibility HandleLaunchAsValidationErrorIconVisibility( ) const
	{
		return EVisibility::Hidden;
	}

	// Callback for getting the foreground text color of the map combo box.
	FSlateColor HandleMapComboBoxColorAndOpacity( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			const FString& InitialMap = RolePtr->GetInitialMap();

			if (InitialMap.IsEmpty() || (AvailableMaps->Contains(InitialMap)))
			{
				return FSlateColor::UseForeground();
			}
		}

		return FLinearColor::Red;
	}

	// Callback for changing the initial map selection.
	void HandleMapComboBoxSelectionChanged( TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo )
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			if (Selection.IsValid() && (Selection != MapList[0]))
			{
				RolePtr->SetInitialMap(*Selection);
			}
			else
			{
				RolePtr->SetInitialMap(FString());
			}
		}
	}

	// Callback for getting the visibility of the validation error icon of the 'Initial Map' combo box.
	EVisibility HandleMapValidationErrorIconVisibility( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			const FString& InitialMap = RolePtr->GetInitialMap();

			if (InitialMap.IsEmpty() || (AvailableMaps->Contains(InitialMap)))
			{
				return EVisibility::Hidden;
			}
		}

		return EVisibility::Visible;
	}

	// Callback for checking the specified 'No VSync' check box.
	void HandleVsyncCheckBoxCheckStateChanged( ECheckBoxState NewState )
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			RolePtr->SetVsyncEnabled(NewState == ECheckBoxState::Checked);
		}
	}

	// Callback for determining the checked state of the specified 'No VSync' check box.
	ECheckBoxState HandleVsyncCheckBoxIsChecked( ) const
	{
		ILauncherProfileLaunchRolePtr RolePtr = Role.Pin();

		if (RolePtr.IsValid())
		{
			if (RolePtr->IsVsyncEnabled())
			{
				return ECheckBoxState::Checked;
			}
		}

		return ECheckBoxState::Unchecked;
	}

private:

	// Holds a pointer to the list of available cultures.
	const TArray<FString>* AvailableCultures;

	// Holds a pointer to the list of available maps.
	const TArray<FString>* AvailableMaps;

	// Holds the command line text box.
	TSharedPtr<SEditableTextBox> CommandLineTextBox;

	// Holds the instance type combo box.
	TSharedPtr<SComboBox<TSharedPtr<ELauncherProfileRoleInstanceTypes::Type> > > InstanceTypeComboBox;

	// Holds the list of instance types.
	TArray<TSharedPtr<ELauncherProfileRoleInstanceTypes::Type> > InstanceTypeList;

	// Holds the culture text box.
	TSharedPtr<STextComboBox> CultureComboBox;

	// Holds the list of cultures that are available for the selected game.
	TArray<TSharedPtr<FString> > CultureList;

	// Holds the map text box.
	TSharedPtr<STextComboBox> MapComboBox;

	// Holds the list of maps that are available for the selected game.
	TArray<TSharedPtr<FString> > MapList;

	// Holds the profile that owns the role being edited.
	TWeakPtr<ILauncherProfile> Profile;

	// Holds a pointer to the role that is being edited in this widget.
	TWeakPtr<ILauncherProfileLaunchRole> Role;
};
