// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Launch/SProjectLauncherLaunchTaskSettings.h"

#include "Fonts/SlateFontInfo.h"
#include "Styling/CoreStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Deploy/SProjectLauncherDeployToDeviceSettings.h"
#include "Widgets/Project/SProjectLauncherProjectPage.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherLaunchTaskSettings"


/* SProjectLauncherLaunchTaskSettings structors
 *****************************************************************************/

SProjectLauncherLaunchTaskSettings::~SProjectLauncherLaunchTaskSettings()
{
}


/* SProjectLauncherLaunchTaskSettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherLaunchTaskSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SScrollBox)

				+ SScrollBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SGridPanel)
							.FillColumn(1, 1.0f)

						// project section
						+ SGridPanel::Slot(0, 0)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Top)
							[
								SNew(STextBlock)
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
									.Text(LOCTEXT("ProjectSectionHeader", "Project"))
							]

						+ SGridPanel::Slot(1, 0)
							.Padding(32.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SProjectLauncherProjectPage, InModel, false)
							]

						// cook section
/*						+ SGridPanel::Slot(0, 3)
							.ColumnSpan(3)
							.Padding(0.0f, 16.0f)
							[
								SNew(SSeparator)
								.Orientation(Orient_Horizontal)
							]

						+ SGridPanel::Slot(0, 4)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Top)
							[
								SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
								.Text(LOCTEXT("CookSectionHeader", "Cook"))
							]

						+ SGridPanel::Slot(1, 4)
							.Padding(32.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SProjectLauncherSimpleCookPage, InModel)
							]*/

						// launch section
						+ SGridPanel::Slot(0, 9)
							.ColumnSpan(3)
							.Padding(0.0f, 16.0f)
							[
								SNew(SSeparator)
									.Orientation(Orient_Horizontal)
							]

						+ SGridPanel::Slot(0, 10)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Top)
							[
								SNew(STextBlock)
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
									.Text(LOCTEXT("TargetSectionHeader", "Target"))
							]

						+ SGridPanel::Slot(1, 10)
							.HAlign(HAlign_Fill)
							.Padding(32.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SProjectLauncherDeployToDeviceSettings, InModel, EVisibility::Hidden)
							]
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
