// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

/**
 * Implements the launcher settings widget.
 */
class SProjectLauncherLaunchTaskSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherLaunchTaskSettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherLaunchTaskSettings( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
