// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Package/SProjectLauncherPackagePage.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Package/SProjectLauncherPackagingSettings.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherPackagePage"


/* SProjectLauncherPackagePage structors
 *****************************************************************************/

SProjectLauncherPackagePage::~SProjectLauncherPackagePage()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherPackagePage interface
 *****************************************************************************/

void SProjectLauncherPackagePage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	// create packing mode menu
	FMenuBuilder PackagingModeMenuBuilder(true, NULL);
	{
		FUIAction LocallyAction(FExecuteAction::CreateSP(this, &SProjectLauncherPackagePage::HandlePackagingModeMenuEntryClicked, ELauncherProfilePackagingModes::Locally));
		PackagingModeMenuBuilder.AddMenuEntry(LOCTEXT("LocallyAction", "Package & store locally"), LOCTEXT("LocallyActionHint", "Store this build locally."), FSlateIcon(), LocallyAction);
		
		FUIAction SharedRepositoryAction(FExecuteAction::CreateSP(this, &SProjectLauncherPackagePage::HandlePackagingModeMenuEntryClicked, ELauncherProfilePackagingModes::SharedRepository));
		PackagingModeMenuBuilder.AddMenuEntry(LOCTEXT("SharedRepositoryAction", "Package & store in repository"), LOCTEXT("SharedRepositoryActionHint", "Store this build in a shared repository."), FSlateIcon(), SharedRepositoryAction);

		FUIAction DoNotPackageAction(FExecuteAction::CreateSP(this, &SProjectLauncherPackagePage::HandlePackagingModeMenuEntryClicked, ELauncherProfilePackagingModes::DoNotPackage));
		PackagingModeMenuBuilder.AddMenuEntry(LOCTEXT("DoNotPackageAction", "Do not package"), LOCTEXT("DoNotPackageActionHint", "Do not package the build at this time."), FSlateIcon(), DoNotPackageAction);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("WhereToStoreBuildText", "How would you like to package the build?"))			
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.0, 0.0, 0.0, 0.0)
					[
						// packaging mode menu
						SNew(SComboButton)
							.ButtonContent()
							[
								SNew(STextBlock)
									.Text(this, &SProjectLauncherPackagePage::HandlePackagingModeComboButtonContentText)
							]
							.ContentPadding(FMargin(6.0, 2.0))
							.MenuContent()
							[
								PackagingModeMenuBuilder.MakeWidget()
							]
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0)
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SAssignNew(ProjectLauncherPackagingSettings, SProjectLauncherPackagingSettings, InModel)
					.Visibility(this, &SProjectLauncherPackagePage::HandlePackagingSettingsAreaVisibility)
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherPackagePage::HandleProfileManagerProfileSelected);
}


/* SProjectLauncherLaunchPage callbacks
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FText SProjectLauncherPackagePage::HandlePackagingModeComboButtonContentText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfilePackagingModes::Type PackagingMode = SelectedProfile->GetPackagingMode();

		if (PackagingMode == ELauncherProfilePackagingModes::DoNotPackage)
		{
			return LOCTEXT("DoNotPackageAction", "Do not package");
		}

		if (PackagingMode == ELauncherProfilePackagingModes::Locally)
		{
			return LOCTEXT("LocallyAction", "Package & store locally");
		}

		if (PackagingMode == ELauncherProfilePackagingModes::SharedRepository)
		{
			return LOCTEXT("SharedRepositoryAction", "Package & store in repository");
		}

		return LOCTEXT("PackagingModeComboButtonDefaultText", "Select...");
	}

	return FText::GetEmpty();
}


void SProjectLauncherPackagePage::HandlePackagingModeMenuEntryClicked(ELauncherProfilePackagingModes::Type PackagingMode)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetPackagingMode(PackagingMode);

		check(ProjectLauncherPackagingSettings.IsValid());
		ProjectLauncherPackagingSettings->UpdatePackagingModeWidgets();
	}
}


EVisibility SProjectLauncherPackagePage::HandlePackagingSettingsAreaVisibility() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfilePackagingModes::Type PackagingMode = SelectedProfile->GetPackagingMode();

		if ((PackagingMode == ELauncherProfilePackagingModes::Locally) || (PackagingMode == ELauncherProfilePackagingModes::SharedRepository))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


void SProjectLauncherPackagePage::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{

}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
