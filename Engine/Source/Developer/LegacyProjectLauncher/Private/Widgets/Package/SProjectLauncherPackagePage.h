// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class SProjectLauncherPackagingSettings;

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherPackagePage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherPackagePage) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherPackagePage( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

private:

	// Callback for getting the content text of the 'Cook Mode' combo button.
	FText HandlePackagingModeComboButtonContentText( ) const;

	// Callback for clicking an item in the 'Cook Mode' menu.
	void HandlePackagingModeMenuEntryClicked( ELauncherProfilePackagingModes::Type PackagingMode );

	// Callback for getting the visibility of the packaging settings area.
	EVisibility HandlePackagingSettingsAreaVisibility( ) const;

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

	TSharedPtr<SProjectLauncherPackagingSettings> ProjectLauncherPackagingSettings;
};
