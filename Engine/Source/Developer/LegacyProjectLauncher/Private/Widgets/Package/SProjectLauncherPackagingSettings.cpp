// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Package/SProjectLauncherPackagingSettings.h"

#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateTypes.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherPackagingSettings"


/* SProjectLauncherPackagingSettings structors
 *****************************************************************************/

SProjectLauncherPackagingSettings::~SProjectLauncherPackagingSettings()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherPackagingSettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherPackagingSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0)
			[
				SNew(SBorder)
					.Padding(8.0)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
									.Text(this, &SProjectLauncherPackagingSettings::HandleDirectoryTitleText)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// repository path text box
										SAssignNew(DirectoryPathTextBox, SEditableTextBox)
										.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
										.OnTextCommitted(this, &SProjectLauncherPackagingSettings::OnTextCommitted)
										.OnTextChanged(this, &SProjectLauncherPackagingSettings::OnTextChanged)
										.HintText(this, &SProjectLauncherPackagingSettings::HandleHintPathText)
									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.Padding(4.0, 0.0, 0.0, 0.0)
									[
										// browse button
										SNew(SButton)
											.ContentPadding(FMargin(6.0, 2.0))
											.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
											.Text(LOCTEXT("BrowseButtonText", "Browse..."))
											.ToolTipText(LOCTEXT("BrowseButtonToolTip", "Browse for the directory"))
											.OnClicked(this, &SProjectLauncherPackagingSettings::HandleBrowseButtonClicked)
									]
							]

						// don't include editor content
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[

								SNew(SCheckBox)
									.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
									.IsChecked(this, &SProjectLauncherPackagingSettings::HandleForDistributionCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SProjectLauncherPackagingSettings::HandleForDistributionCheckBoxCheckStateChanged)
									.Padding(FMargin(4.0f, 0.0f))
									.ToolTipText(LOCTEXT("ForDistributionCheckBoxTooltip", "If checked the build will be marked as for release to the public (distribution)."))
									.Content()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("ForDistributionCheckBoxText", "Is this build for distribution to the public"))
									]
							]


						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SCheckBox)
								.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
								.IsChecked(this, &SProjectLauncherPackagingSettings::HandleIncludePrerequisitesCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherPackagingSettings::HandleIncludePrerequisitesCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("IncludePrerequisitesCheckBoxTooltip", "If checked the build will include the prerequisites installer on platforms that support it."))
								.Content()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("IncludePrerequisitesCheckBoxText", "Include an installer for prerequisites of packaged games"))
								]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SCheckBox)
								.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
								.IsChecked(this, &SProjectLauncherPackagingSettings::HandleUseIoStoreCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherPackagingSettings::HandleUseIoStoreCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("UseIoStoreCheckBoxTooltip", "Use container files for optimized loading (I/O Store)"))
								.Content()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("UseIoStoreCheckBoxText", "Use container files for optimized loading (I/O Store)"))
								]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SCheckBox)
								.IsEnabled(this, &SProjectLauncherPackagingSettings::IsEditable)
								.IsChecked(this, &SProjectLauncherPackagingSettings::HandleMakeBinaryConfigCheckBoxIsChecked)
								.OnCheckStateChanged(this, &SProjectLauncherPackagingSettings::HandleMakeBinaryConfigCheckStateChanged)
								.Padding(FMargin(4.0f, 0.0f))
								.ToolTipText(LOCTEXT("MakeBinaryConfigCheckBoxTooltip", "Bakes the config (.ini, with plugins) data into a binary file, along with optional custom data"))
								.Content()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("MakeBinaryConfigCheckBoxText", "Make a binary config file for faster runtime startup times"))
								]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("RefBlockDbFileNameText", "Optional I/O Store Reference Chunk Database:"))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.Padding(0.0, 0.0, 0.0, 3.0)
									[
										// path textbox for previously compressed containers.
										SAssignNew(ReferenceContainerGlobalFileName, SEditableTextBox)
										.IsEnabled(this, &SProjectLauncherPackagingSettings::IsReferenceChunkDbEditable)
										.OnTextCommitted(this, &SProjectLauncherPackagingSettings::OnRefChunkDbFileNameTextCommitted)
										.OnTextChanged(this, &SProjectLauncherPackagingSettings::OnRefChunkDbFileNameTextChanged)
										.ToolTipText(LOCTEXT("RefBlockDbUtocFileNameToolTip", "The path to a global.utoc in a directory of previously released iostore containers."))
										.HintText(LOCTEXT("RefBlockDbUtocFileNameHint", "<path/to/global.utoc>"))
									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.Padding(4.0, 0.0, 0.0, 0.0)
									[
										// browse button for the global.utoc for a previously staged game.
										SNew(SButton)
										.ContentPadding(FMargin(6.0, 2.0))
										.IsEnabled(this, &SProjectLauncherPackagingSettings::IsReferenceChunkDbEditable)
										.Text(LOCTEXT("BrowseButtonText", "Browse..."))
										.ToolTipText(LOCTEXT("BrowseForRefDbUtocFilenameButtonToolTip", "Browse for the global.utoc file from a previously staged/released game."))
										.OnClicked(this, &SProjectLauncherPackagingSettings::HandleRefChunkDbBrowseButtonClicked)
									]
							] // end reference container line

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.Padding(0.0, 0.0, 0.0, 3.0)
								[
									// path textbox for previously compressed containers' crypto keys
									SAssignNew(ReferenceContainerCryptoKeysFileName, SEditableTextBox)
									.IsEnabled(this, &SProjectLauncherPackagingSettings::IsReferenceChunkDbEditable)
									.OnTextCommitted(this, &SProjectLauncherPackagingSettings::OnRefChunkCryptoFileNameTextCommitted)
									.OnTextChanged(this, &SProjectLauncherPackagingSettings::OnRefChunkCryptoFileNameTextChanged)
									.ToolTipText(LOCTEXT("RefBlockDbCryptoFileNameToolTip", "The path to a crypto.json to decrypt the reference chunk containers, if needed."))
									.HintText(LOCTEXT("RefBlockDbCryptoFileNameHint", "<path/to/crypto.json, optional>"))
								]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.Padding(4.0, 0.0, 0.0, 0.0)
									[
										// browse button for the crypto.json to decrypt the ref block containers, if needed
										SNew(SButton)
										.ContentPadding(FMargin(6.0, 2.0))
										.IsEnabled(this, &SProjectLauncherPackagingSettings::IsReferenceChunkDbEditable)
										.Text(LOCTEXT("BrowseButtonText", "Browse..."))
										.ToolTipText(LOCTEXT("BrowseForRefDbCryptoFilenameButtonToolTip", "Browse for the crypto.json file to decrypt the reference containers, if needed."))
										.OnClicked(this, &SProjectLauncherPackagingSettings::HandleRefChunkCryptoBrowseButtonClicked)
									]
							] // end reference container crypto keys line
					]
			]
	];

	Model->OnProfileSelected().AddSP(this, &SProjectLauncherPackagingSettings::HandleProfileManagerProfileSelected);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherPackagingSettings::UpdatePackagingModeWidgets()
{
	DirectoryPathTextBox->SetText(HandleDirectoryPathText());

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		ReferenceContainerCryptoKeysFileName->SetText(FText::FromString(SelectedProfile->GetReferenceContainerCryptoKeysFileName()));
		ReferenceContainerGlobalFileName->SetText(FText::FromString(SelectedProfile->GetReferenceContainerGlobalFileName()));
	}
}


/* SProjectLauncherPackagingSettings callbacks
 *****************************************************************************/

void SProjectLauncherPackagingSettings::HandleForDistributionCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetForDistribution(NewState == ECheckBoxState::Checked);
	}
}


ECheckBoxState SProjectLauncherPackagingSettings::HandleForDistributionCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsForDistribution())
		{
			return ECheckBoxState::Checked;
		}
	}
	return ECheckBoxState::Unchecked;
}

void SProjectLauncherPackagingSettings::HandleIncludePrerequisitesCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetIncludePrerequisites(NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState SProjectLauncherPackagingSettings::HandleIncludePrerequisitesCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsIncludingPrerequisites())
		{
			return ECheckBoxState::Checked;
		}
	}
	return ECheckBoxState::Unchecked;
}

void SProjectLauncherPackagingSettings::HandleUseIoStoreCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetUseIoStore(NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState SProjectLauncherPackagingSettings::HandleUseIoStoreCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->IsUsingIoStore())
		{
			return ECheckBoxState::Checked;
		}
	}
	return ECheckBoxState::Unchecked;
}

void SProjectLauncherPackagingSettings::HandleMakeBinaryConfigCheckStateChanged(ECheckBoxState NewState)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetMakeBinaryConfig(NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState SProjectLauncherPackagingSettings::HandleMakeBinaryConfigCheckBoxIsChecked() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->MakeBinaryConfig())
		{
			return ECheckBoxState::Checked;
		}
	}
	return ECheckBoxState::Unchecked;
}

FText SProjectLauncherPackagingSettings::HandleDirectoryTitleText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfilePackagingModes::Type PackagingMode = SelectedProfile->GetPackagingMode();
		switch (PackagingMode)
		{
		case ELauncherProfilePackagingModes::Locally:
			return LOCTEXT("LocalDirectoryPathLabel", "Local Directory Path:");
		case ELauncherProfilePackagingModes::SharedRepository:
			return LOCTEXT("RepositoryPathLabel", "Repository Path:");
		}
	}

	return FText::GetEmpty();
}


FText SProjectLauncherPackagingSettings::HandleDirectoryPathText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ELauncherProfilePackagingModes::Type PackagingMode = SelectedProfile->GetPackagingMode();
		if (PackagingMode == ELauncherProfilePackagingModes::Locally)
		{
			return FText::FromString(SelectedProfile->GetPackageDirectory());
		}
	}

	return FText::GetEmpty();
}


FText SProjectLauncherPackagingSettings::HandleHintPathText() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid() && SelectedProfile->GetPackagingMode() == ELauncherProfilePackagingModes::Locally && !SelectedProfile->GetProjectBasePath().IsEmpty())
	{
		const FString ProjectPathWithoutExtension = FPaths::GetPath(SelectedProfile->GetProjectPath());
		return FText::FromString(ProjectPathWithoutExtension / "Saved" / "StagedBuilds");
	}
	
	return FText::GetEmpty();
}


void SProjectLauncherPackagingSettings::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	UpdatePackagingModeWidgets();
}


FReply SProjectLauncherPackagingSettings::HandleBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(),
			DirectoryPathTextBox->GetText().ToString(),
			FolderName
			);

		if (bFolderSelected)
		{
			if (!FolderName.EndsWith(TEXT("/")))
			{
				FolderName += TEXT("/");
			}

			DirectoryPathTextBox->SetText(FText::FromString(FolderName));
			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

			if (SelectedProfile.IsValid())
			{
				SelectedProfile->SetPackageDirectory(FolderName);
			}
		}
	}

	return FReply::Handled();
}


bool SProjectLauncherPackagingSettings::IsEditable() const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->GetPackagingMode() == ELauncherProfilePackagingModes::Locally;
	}
	return false;
}


void SProjectLauncherPackagingSettings::OnTextChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetPackageDirectory(InText.ToString());
	}
}


void SProjectLauncherPackagingSettings::OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetPackageDirectory(InText.ToString());
		}
	}
}



bool SProjectLauncherPackagingSettings::IsReferenceChunkDbEditable() const
{
	if (IsEditable() == false)
	{
		return false;
	}

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		return SelectedProfile->IsUsingIoStore();
	}
	return false;
}

void SProjectLauncherPackagingSettings::OnRefChunkDbFileNameTextChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetReferenceContainerGlobalFileName(InText.ToString());
	}
}


void SProjectLauncherPackagingSettings::OnRefChunkDbFileNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetReferenceContainerGlobalFileName(InText.ToString());
		}
	}
}

FReply SProjectLauncherPackagingSettings::HandleRefChunkDbBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		TArray<FString> SelectedFiles;
		bool bFileSelected = DesktopPlatform->OpenFileDialog(
			ParentWindowHandle, 
			LOCTEXT("GlobalUtocTitle", "Select a global.utoc file").ToString(),
			ReferenceContainerGlobalFileName->GetText().ToString(),
			TEXT(""), 
			TEXT("global.utoc files|global.utoc"),
			EFileDialogFlags::None /* only single file selection */,
			SelectedFiles
			);

		if (bFileSelected && SelectedFiles.Num())
		{
			ReferenceContainerGlobalFileName->SetText(FText::FromString(SelectedFiles[0]));

			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
			if (SelectedProfile.IsValid())
			{
				SelectedProfile->SetReferenceContainerGlobalFileName(SelectedFiles[0]);
			}
		}
	}

	return FReply::Handled();
}


void SProjectLauncherPackagingSettings::OnRefChunkCryptoFileNameTextChanged(const FText& InText)
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->SetReferenceContainerCryptoKeysFileName(InText.ToString());
	}
}


void SProjectLauncherPackagingSettings::OnRefChunkCryptoFileNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
		if (SelectedProfile.IsValid())
		{
			SelectedProfile->SetReferenceContainerCryptoKeysFileName(InText.ToString());
		}
	}
}

FReply SProjectLauncherPackagingSettings::HandleRefChunkCryptoBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		TArray<FString> SelectedFiles;
		bool bFileSelected = DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			LOCTEXT("GlobalUtocKeysTitle", "Select a crypto.json file").ToString(),
			ReferenceContainerCryptoKeysFileName->GetText().ToString(),
			TEXT(""),
			TEXT("crypto.json files|crypto.json"),
			EFileDialogFlags::None /* only single file selection */,
			SelectedFiles
		);

		if (bFileSelected && SelectedFiles.Num())
		{
			ReferenceContainerCryptoKeysFileName->SetText(FText::FromString(SelectedFiles[0]));

			ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
			if (SelectedProfile.IsValid())
			{
				SelectedProfile->SetReferenceContainerCryptoKeysFileName(SelectedFiles[0]);
			}
		}
	}

	return FReply::Handled();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
