// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class SEditableTextBox;
enum class ECheckBoxState : uint8;

/**
 * Implements the packaging settings panel.
 */
class SProjectLauncherPackagingSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherPackagingSettings) { }
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherPackagingSettings( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel );

	// Update any widgets to reflect the change in packaging mode.
	void UpdatePackagingModeWidgets();

private:

	void HandleForDistributionCheckBoxCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleForDistributionCheckBoxIsChecked() const;

	void HandleIncludePrerequisitesCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleIncludePrerequisitesCheckBoxIsChecked() const;

	void HandleUseIoStoreCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleUseIoStoreCheckBoxIsChecked() const;

	void HandleMakeBinaryConfigCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState HandleMakeBinaryConfigCheckBoxIsChecked() const;

	// Reference chunk database handlers.
	bool IsReferenceChunkDbEditable() const;
	void OnRefChunkDbFileNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	void OnRefChunkDbFileNameTextChanged(const FText& InText);
	FReply HandleRefChunkDbBrowseButtonClicked();
	void OnRefChunkCryptoFileNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	void OnRefChunkCryptoFileNameTextChanged(const FText& InText);
	FReply HandleRefChunkCryptoBrowseButtonClicked();


	// Callback for getting the content text of the 'Directory' label.
	FText HandleDirectoryTitleText() const;
	FText HandleDirectoryPathText() const;

	// Callback for getting the hint text which contains the default project output path
	FText HandleHintPathText() const;

	// Callback for changing the selected profile in the profile manager.
	void HandleProfileManagerProfileSelected( const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile );

	FReply HandleBrowseButtonClicked();

	/** Handles entering in a command */
	bool IsEditable() const;

	void OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	void OnTextChanged(const FText& InText);

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

	// Holds the repository path text box.
	TSharedPtr<SEditableTextBox> DirectoryPathTextBox;

	TSharedPtr<SEditableTextBox> ReferenceContainerGlobalFileName;
	TSharedPtr<SEditableTextBox> ReferenceContainerCryptoKeysFileName;
};
