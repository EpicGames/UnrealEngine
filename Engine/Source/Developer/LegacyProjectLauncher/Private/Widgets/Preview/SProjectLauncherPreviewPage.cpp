// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProjectLauncherPreviewPage.h"

#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Deploy/SProjectLauncherSimpleDeviceListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherPreviewPage"


/* SProjectLauncherPreviewPage interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherPreviewPage::Construct( const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel )
{
	Model = InModel;

	ChildSlot
	[
		SNew(SScrollBox)

		+ SScrollBox::Slot()
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SGridPanel)
					.FillColumn(1, 1.0f)

				// build section
				+ SGridPanel::Slot(0, 0)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
							.Text(LOCTEXT("BuildSectionHeader", "Build"))
					]

				+ SGridPanel::Slot(1, 0)
					.Padding(32.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("ProjectLabel", "Project:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// selected project
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleProjectTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoProjectSelected)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("ConfigurationLabel", "Build Configuration:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// selected build configuration
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleBuildConfigurationTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoBuildConfigurationSelected)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("PlatformsLabel", "Platforms:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// build platforms
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleBuildPlatformsTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoPlatformSelected)
											]
									]
							]
					]

				// cook section
				+ SGridPanel::Slot(0, 1)
					.ColumnSpan(3)
					.Padding(0.0f, 16.0f)
					[
						SNew(SSeparator)
							.Orientation(Orient_Horizontal)
					]

				+ SGridPanel::Slot(0, 2)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
							.Text(LOCTEXT("CookSectionHeader", "Cook"))
					]

				+ SGridPanel::Slot(1, 2)
					.Padding(32.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleCookSummaryBoxVisibility, ELauncherProfileCookModes::DoNotCook)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("CookModeLabel", "Cook Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// do not cook
												SNew(STextBlock)
													.Text(LOCTEXT("DoNotCookLabel", "Do not cook"))
											]
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleCookSummaryBoxVisibility, ELauncherProfileCookModes::OnTheFly)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("BuildModeLabel", "Cook Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cook on the fly
												SNew(STextBlock)
													.Text(LOCTEXT("CookOnTheFlyLabel", "On the fly"))
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("CookerOptionsLabel", "Cooker Options:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cooker options
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleCookerOptionsTextBlockText)
											]
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleCookSummaryBoxVisibility, ELauncherProfileCookModes::ByTheBook)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("BuildModeLabel", "Cook Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cook by the book
												SNew(STextBlock)
													.Text(LOCTEXT("CookByTheBookLabel", "By the book"))
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("CulturesBuildLabel", "Cooked Cultures:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cooked cultures
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleCookedCulturesTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoCookedCulturesSelected)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("MapsBuildLabel", "Cooked Maps:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cooked maps
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleCookedMapsTextBlockText)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("CookerOptionsLabel", "Cooker Options:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// cooker options
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleCookerOptionsTextBlockText)
											]
									]
							]
					]
/*
				// package section
				+ SGridPanel::Slot(0, 3)
					.ColumnSpan(3)
					.Padding(0.0f, 16.0f)
					[
						SNew(SSeparator)
							.Orientation(Orient_Horizontal)
					]

				+ SGridPanel::Slot(0, 4)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
							.Text(LOCTEXT("PackageSectionHeader", "Package"))
					]

				+ SGridPanel::Slot(1, 4)
					.Padding(32.0f, 0.0f, 8.0f, 0.0f)
					[
						SNullWidget::NullWidget
					]
*/
				// deploy section
				+ SGridPanel::Slot(0, 5)
					.ColumnSpan(3)
					.Padding(0.0f, 16.0f)
					[
						SNew(SSeparator)
							.Orientation(Orient_Horizontal)
					]

				+ SGridPanel::Slot(0, 6)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
							.Text(LOCTEXT("DeploySectionHeader", "Deploy"))
					]

				+ SGridPanel::Slot(1, 6)
					.Padding(32.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleDeploySummaryBoxVisibility, ELauncherProfileDeploymentModes::DoNotDeploy)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("DeployModeLabel", "Deploy Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// do not cook
												SNew(STextBlock)
													.Text(LOCTEXT("DoNotDeployLabel", "Do not deploy"))
											]
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)
											.Visibility(this, &SProjectLauncherPreviewPage::HandleDeploySummaryBoxVisibility, ELauncherProfileDeploymentModes::CopyToDevice)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("DeployModeLabel", "Deploy Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// copy to device
												SNew(STextBlock)
													.Text(LOCTEXT("CopyToDeviceLabel", "Copy to device"))
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)
											.Visibility(this, &SProjectLauncherPreviewPage::HandleDeploySummaryBoxVisibility, ELauncherProfileDeploymentModes::FileServer)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("DeployModeLabel", "Deploy Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// copy to device
												SNew(STextBlock)
													.Text(LOCTEXT("FileServerLabel", "File server"))
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SVerticalBox)
											.Visibility(this, &SProjectLauncherPreviewPage::HandleDeployDetailsBoxVisibility)

										+ SVerticalBox::Slot()
											.AutoHeight()
											.Padding(0.0f, 8.0f, 0.0f, 0.0f)
											[
												SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
													.AutoWidth()
													[
														SNew(STextBlock)
															.Text(LOCTEXT("DeviceGroupLabel", "Device Group:"))
													]

												+ SHorizontalBox::Slot()
													.FillWidth(1.0f)
													.HAlign(HAlign_Right)
													.Padding(8.0f, 0.0f, 0.0f, 0.0f)
													[
														// copy to device
														SNew(STextBlock)
															.Text(this, &SProjectLauncherPreviewPage::HandleSelectedDeviceGroupTextBlockText)
													]
											]

										+ SVerticalBox::Slot()
											.AutoHeight()
											.Padding(0.0f, 8.0f, 0.0f, 0.0f)
											[
												SNew(SHorizontalBox)

												+ SHorizontalBox::Slot()
													.AutoWidth()
													[
														SNew(STextBlock)
															.Text(LOCTEXT("DeviceListLabel", "Devices:"))
													]

												+ SHorizontalBox::Slot()
													.FillWidth(1.0f)
													.HAlign(HAlign_Right)
													.Padding(8.0f, 0.0f, 0.0f, 0.0f)
													[
														// device list
														SAssignNew(DeviceProxyListView, SListView<TSharedPtr<ITargetDeviceProxy>>)
															.ListItemsSource(&DeviceProxyList)
															.SelectionMode(ESelectionMode::None)
															.OnGenerateRow(this, &SProjectLauncherPreviewPage::HandleDeviceProxyListViewGenerateRow)
															.HeaderRow
															(
																SNew(SHeaderRow)
																	.Visibility(EVisibility::Collapsed)

																+ SHeaderRow::Column("Device")
																	.DefaultLabel(LOCTEXT("DeviceListDeviceColumnHeader", "Device"))
															)
													]
											]
									]
							]
					]

				// launch section
				+ SGridPanel::Slot(0, 7)
					.ColumnSpan(3)
					.Padding(0.0f, 16.0f)
					[
						SNew(SSeparator)
							.Orientation(Orient_Horizontal)
					]

				+ SGridPanel::Slot(0, 8)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
							.Text(LOCTEXT("LaunchSectionHeader", "Launch"))
					]

				+ SGridPanel::Slot(1, 8)
					.Padding(32.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleLaunchSummaryBoxVisibility, ELauncherProfileLaunchModes::DoNotLaunch)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("LaunchModeLabel", "Launch Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// do not cook
												SNew(STextBlock)
													.Text(LOCTEXT("DoNotLaunchLabel", "Do not launch"))
											]
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleLaunchSummaryBoxVisibility, ELauncherProfileLaunchModes::DefaultRole)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("LaunchModeLabel", "Launch Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// do not cook
												SNew(STextBlock)
													.Text(LOCTEXT("LaunchUsingDefaultRoleLabel", "Using default role"))
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("InitialCultureLabel", "Initial Culture:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// initial culture
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleInitialCultureTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::InitialCultureNotAvailable)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("InitialMapLabel", "Initial Map:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// initial map
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleInitialMapTextBlockText)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
													.Visibility(this, &SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::InitialMapNotAvailable)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("CommandLineLabel", "Command Line:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// command line
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleCommandLineTextBlockText)
											]
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(0.0f, 8.0f, 0.0f, 0.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("VsyncLabel", "VSync:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0f)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// command line
												SNew(STextBlock)
													.Text(this, &SProjectLauncherPreviewPage::HandleLaunchVsyncTextBlockText)
											]
									]
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SVerticalBox)
									.Visibility(this, &SProjectLauncherPreviewPage::HandleLaunchSummaryBoxVisibility, ELauncherProfileLaunchModes::CustomRoles)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("LaunchModeLabel", "Launch Mode:"))
											]

										+ SHorizontalBox::Slot()
											.FillWidth(1.0)
											.HAlign(HAlign_Right)
											.Padding(8.0f, 0.0f, 0.0f, 0.0f)
											[
												// do not cook
												SNew(STextBlock)
													.Text(LOCTEXT("UseCustomRolesLabel", "Using custom roles"))
											]
									]
							]
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SProjectLauncherPreviewPage::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	RefreshDeviceProxyList();
}


/* SProjectLauncherDevicesSelector implementation
 *****************************************************************************/

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void SProjectLauncherPreviewPage::RefreshDeviceProxyList( )
{
	DeviceProxyList.Reset();

	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const ILauncherDeviceGroupPtr& DeployedDeviceGroup = SelectedProfile->GetDeployedDeviceGroup();

		if (DeployedDeviceGroup.IsValid())
		{
			const TArray<FString>& Devices = DeployedDeviceGroup->GetDeviceIDs();

			for (int32 Index = 0; Index < Devices.Num(); ++Index)
			{
				DeviceProxyList.Add(Model->GetDeviceProxyManager()->FindOrAddProxy(Devices[Index]));
			}
		}
	}

	DeviceProxyListView->RequestListRefresh();
}


/* SProjectLauncherPreviewPage callbacks
 *****************************************************************************/

FText SProjectLauncherPreviewPage::HandleBuildConfigurationTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(LexToString(SelectedProfile->GetBuildConfiguration()));
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


FText SProjectLauncherPreviewPage::HandleBuildPlatformsTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const TArray<FString>& Platforms = SelectedProfile->GetCookedPlatforms();
		if (Platforms.Num() > 0)
		{
			FTextBuilder Builder;
			for (int32 PlatformIndex = 0; PlatformIndex < Platforms.Num(); ++PlatformIndex)
			{
				Builder.AppendLine(Platforms[PlatformIndex]);
			}
			return Builder.ToText();
		}
		else
		{
			return LOCTEXT("NotSetText", "<not set>");
		}
	}

	return FText::GetEmpty();
}


FText SProjectLauncherPreviewPage::HandleCommandLineTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const FString& CommandLine = SelectedProfile->GetDefaultLaunchRole()->GetUATCommandLine();

		if (CommandLine.IsEmpty())
		{
			return LOCTEXT("EmptyText", "<empty>");
		}

		return FText::FromString( CommandLine );
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


FText SProjectLauncherPreviewPage::HandleCookedCulturesTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const TArray<FString>& CookedCultures = SelectedProfile->GetCookedCultures();
		if (CookedCultures.Num() > 0)
		{
			FTextBuilder Builder;
			for (int32 CookedCultureIndex = 0; CookedCultureIndex < CookedCultures.Num(); ++CookedCultureIndex)
			{
				Builder.AppendLine(CookedCultures[CookedCultureIndex]);
			}
			return Builder.ToText();
		}
		else
		{
			return LOCTEXT("NotSetText", "<not set>");
		}
	}

	return FText::GetEmpty();
}


FText SProjectLauncherPreviewPage::HandleCookedMapsTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const TArray<FString>& CookedMaps = SelectedProfile->GetCookedMaps();
		if (CookedMaps.Num() > 0)
		{
			FTextBuilder Builder;
			for (int32 CookedMapIndex = 0; CookedMapIndex < CookedMaps.Num(); ++CookedMapIndex)
			{
				Builder.AppendLine(CookedMaps[CookedMapIndex]);
			}
			return Builder.ToText();
		}
		else
		{
			return LOCTEXT("NotSetText", "<not set>");
		}
	}

	return FText::GetEmpty();
}


FText SProjectLauncherPreviewPage::HandleCookerOptionsTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookOptions().IsEmpty())
		{
			return LOCTEXT("NoneText", "<none>");
		}

		return FText::FromString(SelectedProfile->GetCookOptions());
	}

	return FText::GetEmpty();
}


EVisibility SProjectLauncherPreviewPage::HandleCookSummaryBoxVisibility( ELauncherProfileCookModes::Type CookMode ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetCookMode() == CookMode)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherPreviewPage::HandleDeployDetailsBoxVisibility( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


EVisibility SProjectLauncherPreviewPage::HandleDeploySummaryBoxVisibility( ELauncherProfileDeploymentModes::Type DeploymentMode ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDeploymentMode() == DeploymentMode)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


TSharedRef<ITableRow> SProjectLauncherPreviewPage::HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Model->GetSelectedProfile().IsValid());

	ILauncherDeviceGroupPtr DeployedDeviceGroup = Model->GetSelectedProfile()->GetDeployedDeviceGroup();

	check(DeployedDeviceGroup.IsValid());

	return SNew(SProjectLauncherSimpleDeviceListRow, Model.ToSharedRef(), OwnerTable)
		.DeviceProxy(InItem);
}


FText SProjectLauncherPreviewPage::HandleInitialCultureTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const FString& InitialCulture = SelectedProfile->GetDefaultLaunchRole()->GetInitialCulture();

		if (InitialCulture.IsEmpty())
		{
			return LOCTEXT("DefaultText", "<default>");
		}

		return FText::FromString(InitialCulture);
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


FText SProjectLauncherPreviewPage::HandleInitialMapTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		const FString& InitialMap = SelectedProfile->GetDefaultLaunchRole()->GetInitialMap();

		if (InitialMap.IsEmpty())
		{
			return LOCTEXT("DefaultText", "<default>");
		}

		return FText::FromString(InitialMap);
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


EVisibility SProjectLauncherPreviewPage::HandleLaunchSummaryBoxVisibility( ELauncherProfileLaunchModes::Type LaunchMode ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetLaunchMode() == LaunchMode)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherPreviewPage::HandleLaunchVsyncTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetDefaultLaunchRole()->IsVsyncEnabled())
		{
			return LOCTEXT("YesText", "Yes");
		}

		return LOCTEXT("NoText", "No");
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


EVisibility SProjectLauncherPreviewPage::HandlePackageSummaryBoxVisibility( ELauncherProfilePackagingModes::Type PackagingMode ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->GetPackagingMode() == PackagingMode)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherPreviewPage::HandleProjectTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		FString GameName = SelectedProfile->GetProjectName();

		if (GameName.IsEmpty())
		{
			return LOCTEXT("NotSetText", "<not set>");
		}

		return FText::FromString(GameName);
	}

	return LOCTEXT("NotAvailableText", "n/a");
}


FText SProjectLauncherPreviewPage::HandleSelectedDeviceGroupTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		ILauncherDeviceGroupPtr SelectedGroup = SelectedProfile->GetDeployedDeviceGroup();

		if (SelectedGroup.IsValid())
		{
			return FText::FromString(SelectedGroup->GetName());
		}
	}
		
	return LOCTEXT("NoneText", "<none>");
}


FText SProjectLauncherPreviewPage::HandleSelectedProfileTextBlockText( ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		return FText::FromString(SelectedProfile->GetName());
	}

	return LOCTEXT("NoneText", "<none>");
}


EVisibility SProjectLauncherPreviewPage::HandleValidationErrorIconVisibility( ELauncherProfileValidationErrors::Type Error ) const
{
	ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();

	if (SelectedProfile.IsValid())
	{
		if (SelectedProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
