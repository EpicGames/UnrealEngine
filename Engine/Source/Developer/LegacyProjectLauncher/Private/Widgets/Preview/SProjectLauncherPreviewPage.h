// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Models/LegacyProjectLauncherModel.h"

class ITargetDeviceProxy;


/**
 * Implements the launcher's preview page.
 */
class SProjectLauncherPreviewPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherPreviewPage) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

public:

	//~ SCompoundWidget interface

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	/**
	 * Refreshes the list of device proxies.
	 */
	void RefreshDeviceProxyList();

private:

	/** Callback for getting the text in the 'Build Configuration' text block. */
	FText HandleBuildConfigurationTextBlockText() const;

	/** Callback for getting the list of platforms to build for. */
	FText HandleBuildPlatformsTextBlockText() const;

	/** Callback for getting the text in the 'Command Line' text block. */
	FText HandleCommandLineTextBlockText() const;

	/** Callback for getting the text in the 'Cooked Cultures' text block. */
	FText HandleCookedCulturesTextBlockText() const;

	/** Callback for getting the text in the 'Cooked Maps' text block. */
	FText HandleCookedMapsTextBlockText() const;

	/** Callback for getting the text in the 'Cooker Options' text block. */
	FText HandleCookerOptionsTextBlockText() const;

	/** Callback for getting the visibility of the specified cook summary box. */
	EVisibility HandleCookSummaryBoxVisibility(ELauncherProfileCookModes::Type CookMode) const;

	/** Callback for getting the visibility of the deploy details box. */
	EVisibility HandleDeployDetailsBoxVisibility() const;

	/** Callback for getting the visibility of the specified deploy summary box. */
	EVisibility HandleDeploySummaryBoxVisibility(ELauncherProfileDeploymentModes::Type DeploymentMode) const;

	/** Callback for generating a row for the device list. */
	TSharedRef<ITableRow> HandleDeviceProxyListViewGenerateRow(TSharedPtr<ITargetDeviceProxy> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the text in the 'Initial Culture' text block. */
	FText HandleInitialCultureTextBlockText() const;

	/** Callback for getting the text in the 'Initial Map' text block. */
	FText HandleInitialMapTextBlockText() const;

	/** Callback for getting the visibility of the specified cook summary box. */
	EVisibility HandleLaunchSummaryBoxVisibility(ELauncherProfileLaunchModes::Type LaunchMode) const;

	/** Callback for getting the text in the 'VSync' text block. */
	FText HandleLaunchVsyncTextBlockText() const;

	/** Callback for getting the visibility of the specified cook summary box. */
	EVisibility HandlePackageSummaryBoxVisibility(ELauncherProfilePackagingModes::Type PackagingMode) const;

	/** Callback for getting the text in the 'Game' text block. */
	FText HandleProjectTextBlockText() const;

	/** Callback for getting the name of the selected device group. */
	FText HandleSelectedDeviceGroupTextBlockText() const;

	/** Callback for getting the name of the selected profile. */
	FText HandleSelectedProfileTextBlockText() const;

	/** Callback for determining the visibility of a validation error icon. */
	EVisibility HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const;

private:

	/** The list of available device proxies. */
	TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxyList;

	/** The device proxy list view. */
	TSharedPtr<SListView<TSharedPtr<ITargetDeviceProxy>> > DeviceProxyListView;

	/** Pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;
};
