// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/Views/STableViewBase.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Shared/ProjectLauncherDelegates.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Shared/SProjectLauncherProfileLaunchButton.h"
#include "Widgets/Shared/SProjectLauncherProfileNameDescEditor.h"
#include "Widgets/Layout/SSeparator.h"
#include "SSimpleButton.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherSimpleDeviceListRow"

/**
 * Implements a row widget for the launcher's device proxy list.
 */
class SProjectLauncherProfileListRow
	: public STableRow<ILauncherProfilePtr>
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProfileListRow) { }
		
		/**
		 * The Callback for when the edit button is clicked.
		 */
		SLATE_EVENT(FOnProfileRun, OnProfileEdit)
			
		/**
		 * The Callback for when the launch button is clicked.
		 */
		SLATE_EVENT(FOnProfileRun, OnProfileRun)

		/**
		 * The device proxy shown in this row.
		 */
		 SLATE_ARGUMENT(ILauncherProfilePtr, LaunchProfile)

	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs - The construction arguments.
	 * @param InModel - The launcher model this list uses.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<ILauncherProfilePtr>::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(false)
			.Style(FAppStyle::Get(), "TableView.Row"),
			InOwnerTableView
			);

		Model = InModel;
		OnProfileEdit = InArgs._OnProfileEdit;
		OnProfileRun = InArgs._OnProfileRun;
		LaunchProfile = InArgs._LaunchProfile;

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(1)
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SAssignNew(ProfileNameDescEditor, SProjectLauncherProfileNameDescEditor, InModel, false)
						.LaunchProfile(this, &SProjectLauncherProfileListRow::GetLaunchProfile)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4, 0, 0, 0)
					[
						SNew(SSimpleButton)
						.OnClicked(this, &SProjectLauncherProfileListRow::OnEditClicked)
						.ToolTipText(LOCTEXT("EditProfileToolTipText", "Edit profile."))
						.Icon(this, &SProjectLauncherProfileListRow::GetEditIcon)
						.IsEnabled_Lambda( [this]() { return LaunchProfile.IsValid() && LaunchProfile->SupportsLegacyProjectLauncher(); } )
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(12, 5, 0, 5)
					[
						SNew(SSeparator)
						.Orientation(Orient_Vertical)
						.Thickness(1.f)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(24, 0, 20, 0)
					[
						SNew(SProjectLauncherProfileLaunchButton, false)
						.LaunchProfile(this, &SProjectLauncherProfileListRow::GetLaunchProfile)
						.OnClicked(this, &SProjectLauncherProfileListRow::OnRunClicked)
					]
					/*
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4, 0, 0, 0)
					[
						SNew(SComboButton)
						.ComboButtonStyle(FAppStyle::Get(), "ContentBrowser.NewAsset.Style")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(0)
						.OnGetMenuContent(this, &SContentBrowser::MakeCreateAssetContextMenu)
						.ToolTipText(this, &SContentBrowser::GetNewAssetToolTipText)
					]*/
				]
			]
		];
	}

	/**
	 * Triggers a name edit for the profile this row displays.
	 */
	void TriggerNameEdit()
	{
		if (ProfileNameDescEditor.IsValid())
		{
			ProfileNameDescEditor->TriggerNameEdit();
		}
	}

private:

	FReply OnEditClicked()
	{
		if (OnProfileEdit.IsBound())
		{
			OnProfileEdit.Execute(LaunchProfile.ToSharedRef());
		}

		return FReply::Handled();
	}

	FReply OnRunClicked()
	{
		if (OnProfileRun.IsBound())
		{
			OnProfileRun.Execute(LaunchProfile.ToSharedRef());
		}

		return FReply::Handled();
	}

	ILauncherProfilePtr GetLaunchProfile() const
	{
		return LaunchProfile;
	}

	// Get the SlateIcon for Launch Button
	const FSlateBrush* GetEditIcon() const
	{
		return FAppStyle::Get().GetBrush("Icons.Toolbar.Settings");
	}

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

	// Holds a reference to the launch profile that is displayed in this row.
	ILauncherProfilePtr LaunchProfile;

	// Holds a delegate to be invoked when a profile is to be edited.
	FOnProfileRun OnProfileEdit;

	// Holds a delegate to be invoked when a profile is run.
	FOnProfileRun OnProfileRun;

	// Holds a pointer to the name / description editor.
	TSharedPtr<SProjectLauncherProfileNameDescEditor> ProfileNameDescEditor;
};


#undef LOCTEXT_NAMESPACE
