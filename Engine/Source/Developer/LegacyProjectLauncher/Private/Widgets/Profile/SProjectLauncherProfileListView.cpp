// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Profile/SProjectLauncherProfileListView.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SScrollBorder.h"

#include "Models/ProjectLauncherCommands.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Profile/SProjectLauncherProfileListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherProfileListView"


/* SProjectLauncherDeployTargets structors
 *****************************************************************************/

SProjectLauncherProfileListView::SProjectLauncherProfileListView()
	: CommandList(MakeShareable(new FUICommandList))
{
}

SProjectLauncherProfileListView::~SProjectLauncherProfileListView()
{
	if (Model.IsValid())
	{
		const ILauncherProfileManagerRef& ProfileManager = Model->GetProfileManager();
		ProfileManager->OnProfileAdded().RemoveAll(this);
		ProfileManager->OnProfileRemoved().RemoveAll(this);
	}
}


/* SProjectLauncherDeployTargets interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherProfileListView::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	CreateCommands();

	OnProfileEdit = InArgs._OnProfileEdit;
	OnProfileRun = InArgs._OnProfileRun;
	OnProfileDelete = InArgs._OnProfileDelete;

	Model = InModel;

	const ILauncherProfileManagerRef& ProfileManager = Model->GetProfileManager();

	SAssignNew(LaunchProfileListView, SListView<ILauncherProfilePtr>)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&(ProfileManager->GetAllProfiles()))
		.OnGenerateRow(this, &SProjectLauncherProfileListView::HandleProfileListViewGenerateRow)
		.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SProjectLauncherProfileListView::MakeProfileContextMenu));

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBorder, LaunchProfileListView.ToSharedRef())
				[
					LaunchProfileListView.ToSharedRef()
				]
			]
	];

	ProfileManager->OnProfileAdded().AddSP(this, &SProjectLauncherProfileListView::HandleProfileManagerProfileAdded);
	ProfileManager->OnProfileRemoved().AddSP(this, &SProjectLauncherProfileListView::HandleProfileManagerProfileRemoved);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SProjectLauncherProfileListView::CreateCommands()
{
	const FProjectLauncherCommands& Commands = FProjectLauncherCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.RenameProfile, 
		FExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleRenameProfileCommandExecute),
		FCanExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleRenameProfileCommandCanExecute));

	//ActionList.MapAction(Commands.DuplicateProfile,
	//	FExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleDuplicateProfileCommandExecute),
	//	FCanExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleDuplicateProfileCommandCanExecute));

	ActionList.MapAction(Commands.DeleteProfile,
		FExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleDeleteProfileCommandExecute),
		FCanExecuteAction::CreateRaw(this, &SProjectLauncherProfileListView::HandleDeleteProfileCommandCanExecute));
}


/* SProjectLauncherDeployTargets implementation
 *****************************************************************************/

void SProjectLauncherProfileListView::RefreshLaunchProfileList()
{
	LaunchProfileListView->RequestListRefresh();
}


/* SProjectLauncherDeployTargets callbacks
 *****************************************************************************/

bool SProjectLauncherProfileListView::HandleProfileRowIsEnabled(ILauncherProfilePtr LaunchProfile) const
{
	return true;
}


FText SProjectLauncherProfileListView::HandleDeviceListRowToolTipText(ILauncherProfilePtr LaunchProfile) const
{
	return FText();
}


TSharedRef<ITableRow> SProjectLauncherProfileListView::HandleProfileListViewGenerateRow(ILauncherProfilePtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SProjectLauncherProfileListRow, Model.ToSharedRef(), OwnerTable)
		.OnProfileEdit(OnProfileEdit)
		.OnProfileRun(OnProfileRun)
		.LaunchProfile(InItem)
		.IsEnabled(this, &SProjectLauncherProfileListView::HandleProfileRowIsEnabled, InItem);
}


void SProjectLauncherProfileListView::HandleProfileManagerProfileAdded(const ILauncherProfileRef& AddedProfile)
{
	RefreshLaunchProfileList();
}


void SProjectLauncherProfileListView::HandleProfileManagerProfileRemoved(const ILauncherProfileRef& RemovedProfile)
{
	RefreshLaunchProfileList();
}


TSharedPtr<SWidget> SProjectLauncherProfileListView::MakeProfileContextMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	const FProjectLauncherCommands& Commands = FProjectLauncherCommands::Get();
	MenuBuilder.AddMenuEntry(Commands.RenameProfile);
	MenuBuilder.AddMenuEntry(Commands.DeleteProfile);

	return MenuBuilder.MakeWidget();
}


bool SProjectLauncherProfileListView::HandleRenameProfileCommandCanExecute() const
{
	return true;
}


void SProjectLauncherProfileListView::HandleRenameProfileCommandExecute()
{
	TArray<ILauncherProfilePtr> ProfileList = LaunchProfileListView->GetSelectedItems();
	if (ProfileList.Num() >= 1)
	{
		TSharedPtr<ITableRow> TableRow = LaunchProfileListView->WidgetFromItem(ProfileList[0]);
		if (TableRow.IsValid())
		{
			TSharedPtr<SProjectLauncherProfileListRow> ProfileListRow = StaticCastSharedPtr<SProjectLauncherProfileListRow>(TableRow);
			if (ProfileListRow.IsValid())
			{
				ProfileListRow->TriggerNameEdit();
			}
		}
	}
}


bool SProjectLauncherProfileListView::HandleDuplicateProfileCommandCanExecute() const
{
	return true;
}


void SProjectLauncherProfileListView::HandleDuplicateProfileCommandExecute()
{
	TArray<ILauncherProfilePtr> ProfileList = LaunchProfileListView->GetSelectedItems();
	for (auto CurProfile : ProfileList)
	{
		const ILauncherProfileManagerRef& ProfileManager = Model->GetProfileManager();
		// @Todo: Duplicating isn't trivial will implement this when i have time.
	}
}


bool SProjectLauncherProfileListView::HandleDeleteProfileCommandCanExecute() const
{
	return OnProfileDelete.IsBound();
}


void SProjectLauncherProfileListView::HandleDeleteProfileCommandExecute()
{
	if (!OnProfileDelete.IsBound())
	{
		return;
	}
	
	TArray<ILauncherProfilePtr> ProfileList = LaunchProfileListView->GetSelectedItems();

	FText Prompt;
	if (ProfileList.Num() == 1)
	{
		Prompt = FText::Format(LOCTEXT("ProfileDeleteConfirm_Single", "Delete {0}?"), FText::FromString(ProfileList[0]->GetName()));
	}
	else
	{
		Prompt = FText::Format(LOCTEXT("ProfileDeleteConfirm_Multiple", "Delete {0} profiles?"), FText::AsNumber(ProfileList.Num()));
	}

	if (FMessageDialog::Open(EAppMsgType::OkCancel, Prompt) == EAppReturnType::Ok)
	{
		for (auto CurProfile : ProfileList)
		{
			OnProfileDelete.Execute(CurProfile.ToSharedRef());
		}
	}
}


#undef LOCTEXT_NAMESPACE
