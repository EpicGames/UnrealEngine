// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "Widgets/Shared/ProjectLauncherDelegates.h"

class FProjectLauncherModel;
class FUICommandList;
class ITableRow;
class STableViewBase;


/**
 * Implements the deployment targets panel.
 */
class SProjectLauncherProfileListView
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProfileListView) { }
		/**
		 * The Callback for when a profile is to be edited.
		 */
		SLATE_EVENT(FOnProfileRun, OnProfileEdit)

		/**
		 * The Callback for when a profile is to be run.
		 */
		SLATE_EVENT(FOnProfileRun, OnProfileRun)

		/**
		* The Callback for when a profile is to be deleted.
		*/
		SLATE_EVENT(FOnProfileRun, OnProfileDelete)
	SLATE_END_ARGS()

public:

	/** Constructor. */
	SProjectLauncherProfileListView();

	/** Destructor. */
	~SProjectLauncherProfileListView();

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/** Refreshe the list of launch profiles. */
	void RefreshLaunchProfileList();

private:

	/** Builds the command list for the context menu on list items. */
	void CreateCommands();

	/** Callback for getting the enabled state of a profile list row. */
	bool HandleProfileRowIsEnabled(ILauncherProfilePtr LaunchProfile) const;

	/** Callback for getting the tool tip text of a profile list row. */
	FText HandleDeviceListRowToolTipText(ILauncherProfilePtr LaunchProfile) const;

	/** Callback for generating a row in the profile list view. */
	TSharedRef<ITableRow> HandleProfileListViewGenerateRow(ILauncherProfilePtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Callback for when a launch profile has been added to the profile manager. */
	void HandleProfileManagerProfileAdded(const ILauncherProfileRef& AddedProfile);

	/** Callback for when a launch profile has been removed to the profile manager. */
	void HandleProfileManagerProfileRemoved(const ILauncherProfileRef& RemovedProfile);

	/** Callback for creating a context menu for the list. */
	TSharedPtr<SWidget> MakeProfileContextMenu();

	/** Callback returns true if the rename command can be executed. */
	bool HandleRenameProfileCommandCanExecute() const;

	/** Callback to execute the rename command from the context menu. */
	void HandleRenameProfileCommandExecute();

	/** Callback returns true if the duplicate command can be executed. */
	bool HandleDuplicateProfileCommandCanExecute() const;

	/** Callback to execute the duplicate command from the context menu. */
	void HandleDuplicateProfileCommandExecute();

	/** Callback returns true if the delete command can be executed. */
	bool HandleDeleteProfileCommandCanExecute() const;

	/** Callback to execute the delete command from the context menu. */
	void HandleDeleteProfileCommandExecute();

private:

	/** The launch profile list view. */
	TSharedPtr<SListView<ILauncherProfilePtr> > LaunchProfileListView;

	/** Pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** A delegate to be invoked when a profile is to be edited. */
	FOnProfileRun OnProfileEdit;

	/** A delegate to be invoked when a profile is run. */
	FOnProfileRun OnProfileRun;

	/** A delegate to be invoked when a profile is deleted. */
	FOnProfileRun OnProfileDelete;

	/** Commands handled by this widget. */
	TSharedRef<FUICommandList> CommandList;

};
