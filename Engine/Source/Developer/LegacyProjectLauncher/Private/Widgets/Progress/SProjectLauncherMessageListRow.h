// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherMessageListRow"

class Error;

struct FProjectLauncherMessage
{
	FText Message;
	ELogVerbosity::Type Verbosity;

	FProjectLauncherMessage(const FText& NewMessage, ELogVerbosity::Type InVerbosity)
		: Message(NewMessage)
		, Verbosity(InVerbosity)
	{ }
};


/**
 * Implements a row widget for the launcher's task list.
 */
class SProjectLauncherMessageListRow
	: public SMultiColumnTableRow<TSharedPtr<FProjectLauncherMessage>>
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherMessageListRow) { }
		SLATE_ARGUMENT(TSharedPtr<FProjectLauncherMessage>, Message)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceProxyManager The device proxy manager to use.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Message = InArgs._Message;

		SMultiColumnTableRow<TSharedPtr<FProjectLauncherMessage>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "Status")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign((VAlign_Center))
				[
					SNew(STextBlock)
						.ColorAndOpacity(HandleGetTextColor())
						.Text(Message->Message)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	// Callback for getting the task's status text.
	FSlateColor HandleGetTextColor( ) const
	{
		if ((Message->Verbosity == ELogVerbosity::Error) ||
			(Message->Verbosity == ELogVerbosity::Fatal))
		{
			return FLinearColor::Red;
		}
		else if (Message->Verbosity == ELogVerbosity::Warning)
		{
			return FLinearColor::Yellow;
		}
		else
		{
			return FSlateColor::UseForeground();
		}
	}

private:

	// Holds a pointer to the task that is displayed in this row.
	TSharedPtr<FProjectLauncherMessage> Message;
};


#undef LOCTEXT_NAMESPACE
