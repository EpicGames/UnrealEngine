// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ILauncherWorker.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Types/SlateStructs.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopeLock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "DesktopPlatformModule.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"

#include "Widgets/Progress/SProjectLauncherTaskListRow.h"
#include "Widgets/Progress/SProjectLauncherMessageListRow.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherProgress"


/**
 * Implements the launcher's progress page.
 */
class SProjectLauncherProgress
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProgress) { }
	SLATE_EVENT(FOnClicked, OnCloseClicked)
	SLATE_EVENT(FOnClicked, OnRerunClicked)
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherProgress( ) = default;

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct( const FArguments& InArgs )
	{
		OnCloseClicked = InArgs._OnCloseClicked;
		OnRerunClicked = InArgs._OnRerunClicked;

		TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
			.Orientation(EOrientation::Orient_Horizontal)
			.AlwaysShowScrollbar(true);
		TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
			.Orientation(EOrientation::Orient_Vertical)
			.AlwaysShowScrollbar(true);

		MaxMessageListRowWidth = 0.0f;

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0, 16.0, 16.0, 0.0)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(this, &SProjectLauncherProgress::GetSelectedProfileNameText)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(this, &SProjectLauncherProgress::HandleProgressTextBlockText)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0, 4.0, 0.0, 0.0)
				[
					SAssignNew(ProgressBar, SProgressBar)
					.Percent(this, &SProjectLauncherProgress::HandleProgressBarPercent)
				]
			]

			+ SVerticalBox::Slot()
			.Padding(0.0, 8.0, 0.0, 0.0)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				
				+ SSplitter::Slot()
				.Value(0.33f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(0.0f)
					[
						SAssignNew(TaskListView, SListView<ILauncherTaskPtr>)
						.HeaderRow
						(
							SNew(SHeaderRow)

							+ SHeaderRow::Column("Icon")
							.DefaultLabel(LOCTEXT("TaskListIconColumnHeader", " "))
							.FixedWidth(20.0)

							+ SHeaderRow::Column("Task")
							.DefaultLabel(LOCTEXT("TaskListTaskColumnHeader", "Task"))
							.FillWidth(1.0)

							+ SHeaderRow::Column("Warnings")
							.DefaultLabel(LOCTEXT("TaskListWarningsColumnHeader", "Warnings"))
							.FixedWidth(64.0)

							+ SHeaderRow::Column("Errors")
							.DefaultLabel(LOCTEXT("TaskListErrorsColumnHeader", "Errors"))
							.FixedWidth(64.0)

							+ SHeaderRow::Column("Duration")
							.DefaultLabel(LOCTEXT("TaskListDurationColumnHeader", "Duration"))
							.FixedWidth(64.0)

							+ SHeaderRow::Column("Status")
							.DefaultLabel(LOCTEXT("TaskListStatusColumnHeader", "Status"))
							.FixedWidth(80.0)
						)
						.ListItemsSource(&TaskList)
						.OnGenerateRow(this, &SProjectLauncherProgress::HandleTaskListViewGenerateRow)
						.SelectionMode(ESelectionMode::Single)
					]
				]

				//content area for the log
				+ SSplitter::Slot()
				.Value(0.66f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(0.0f)
					[
						SNew(SGridPanel)
						.FillColumn(0, 1.f)
						.FillRow(1, 1.f)
						+ SGridPanel::Slot(0, 0)
						[
							SNew(SHeaderRow)
							+ SHeaderRow::Column("Status")
							.DefaultLabel(LOCTEXT("TaskListOutputLogColumnHeader", "Output Log"))
							.FillWidth(1.0)
						]
						+ SGridPanel::Slot(1, 0)
						[
							SNew(SHeaderRow)
						]
						+ SGridPanel::Slot(0, 1)
						[
							SAssignNew(HorizontalScrollBox, SScrollBox)
							.Orientation(EOrientation::Orient_Horizontal)
							.ExternalScrollbar(HorizontalScrollBar)
							+ SScrollBox::Slot()
							[
								SNew(SBox)
								.Padding(0.0f)
								.MinDesiredWidth_Lambda([this]()
								{
									// Cache the max desired width seen so far of rows in the message list.
									// We use this to prevent the horizontal scroll bar from constantly shrinking/expanding as rows are virtualized.
									// This will get reset whenever the message list is cleared.
									MaxMessageListRowWidth = FMath::Max(MaxMessageListRowWidth, (float)MessageListView->GetDesiredSize().X);

									// Make the message list at least as wide as the scroll box so that there's not an empty space to its right;
									// scroll boxes do not deal with alignment in the scroll direction (horizontal in this case).
									const float CurrentScrollWidth = (float)HorizontalScrollBox->GetTickSpaceGeometry().GetLocalSize().X;
									return FMath::Max(MaxMessageListRowWidth, CurrentScrollWidth);
								})
								[
									SAssignNew(MessageListView, SListView< TSharedPtr<FProjectLauncherMessage> >)
									.HeaderRow
									(
										SNew(SHeaderRow)
										.Visibility(EVisibility::Collapsed)
										+ SHeaderRow::Column("Status")
										.DefaultLabel(LOCTEXT("TaskListOutputLogColumnHeader", "Output Log"))
									)
									.ListItemsSource(&MessageList)
									.OnGenerateRow(this, &SProjectLauncherProgress::HandleMessageListViewGenerateRow)
									.SelectionMode(ESelectionMode::Multi)
									.ExternalScrollbar(VerticalScrollBar)
									.AllowOverscroll(EAllowOverscroll::No)
									.ConsumeMouseWheel(EConsumeMouseWheel::Always)
								]
							]
						]
						+ SGridPanel::Slot(1, 1)
						[
							SNew(SBox)
							.WidthOverride(FOptionalSize(16))
							[
								VerticalScrollBar
							]
						]
						+ SGridPanel::Slot(0, 2)
						[
							SNew(SBox)
							.HeightOverride(FOptionalSize(16))
							[
								HorizontalScrollBar
							]
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					// copy button
					SAssignNew(CopyButton, SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(false)
					.Text(LOCTEXT("CopyButtonText", "Copy"))
					.ToolTipText(LOCTEXT("CopyButtonTooltip", "Copy the selected log messages to the clipboard"))
					.OnClicked(this, &SProjectLauncherProgress::HandleCopyButtonClicked)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					// clear button
					SAssignNew(ClearButton, SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(false)
					.Text(LOCTEXT("ClearButtonText", "Clear Log"))
					.ToolTipText(LOCTEXT("ClearButtonTooltip", "Clear the log window"))
					.OnClicked(this, &SProjectLauncherProgress::HandleClearButtonClicked)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					// save button
					SAssignNew(SaveButton, SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(false)
					.Text(LOCTEXT("ExportButtonText", "Save Log..."))
					.ToolTipText(LOCTEXT("SaveButtonTooltip", "Save the entire log to a file"))
					.Visibility((FDesktopPlatformModule::Get() != NULL) ? EVisibility::Visible : EVisibility::Collapsed)
					.OnClicked(this, &SProjectLauncherProgress::HandleSaveButtonClicked)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					// Re-Run button
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(this, &SProjectLauncherProgress::IsRerunButtonEnabled)
					.OnClicked(this, &SProjectLauncherProgress::HandleRerunButtonClicked)
					.ToolTipText(this, &SProjectLauncherProgress::GetRerunButtonToolTip)
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherProgress::GetRerunButtonText)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					// Cancel / Done button
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(this, &SProjectLauncherProgress::IsDoneButtonEnabled)
					.OnClicked(this, &SProjectLauncherProgress::HandleDoneButtonClicked)
					.ToolTipText(this, &SProjectLauncherProgress::GetDoneButtonToolTip)
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherProgress::GetDoneButtonText)
					]
				]
			]
		];
	}

	/**
	 * Sets the launcher worker to track the progress for.
	 *
	 * @param Worker The launcher worker.
	 */
	void SetLauncherWorker( const ILauncherWorkerRef& Worker )
	{
		LauncherWorker = Worker;

		Worker->GetTasks(TaskList);
		TaskListView->RequestListRefresh();

		MessageList.Reset();
		MaxMessageListRowWidth = 0.0;
		Worker->OnOutputReceived().AddRaw(this, &SProjectLauncherProgress::HandleOutputReceived);
		MessageListView->RequestListRefresh();
	}

	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		if (PendingMessages.Num() > 0)
		{
			FScopeLock ScopedLock(&CriticalSection);
			for (int32 Index = 0; Index < PendingMessages.Num(); ++Index)
			{
				MessageList.Add(PendingMessages[Index]);
			}
			PendingMessages.Reset();
			MessageListView->RequestListRefresh();

			// only scroll when at the end of the listview
			if (FMath::IsNearlyEqual(MessageListView->GetScrollDistanceRemaining().Y, 0.0f, 1.e-7f))
			{
				MessageListView->RequestScrollIntoView(MessageList.Last());
			}
		}
		SaveButton->SetEnabled(MessageList.Num() > 0);
		ClearButton->SetEnabled(MessageList.Num() > 0);
		CopyButton->SetEnabled(MessageListView->GetNumItemsSelected() > 0);
	}

private:

	void HandleOutputReceived(const FString& InMessage)
	{
		FScopeLock ScopedLock(&CriticalSection);
		ELogVerbosity::Type Verbosity = ELogVerbosity::Log;

		if (InMessage.Contains(TEXT("Automation.ParseCommandLine:"), ESearchCase::CaseSensitive))
		{
			Verbosity = ELogVerbosity::Display;
		}
		else if ( InMessage.Contains(TEXT("Error:"), ESearchCase::IgnoreCase) )
		{
			Verbosity = ELogVerbosity::Error;
		}
		else if ( InMessage.Contains(TEXT("Warning:"), ESearchCase::IgnoreCase) )
		{
			Verbosity = ELogVerbosity::Warning;
		}
		TSharedPtr<FProjectLauncherMessage> Message = MakeShareable(new FProjectLauncherMessage(FText::FromString(InMessage), Verbosity));
		PendingMessages.Add(Message);
	}

	// Callback for getting the filled percentage of the progress bar.
	TOptional<float> HandleProgressBarPercent( ) const
	{
		if (TaskList.Num() > 0)
		{
			ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

			if (LauncherWorkerPtr.IsValid())
			{
				int32 NumFinished = 0;

				for (int32 TaskIndex = 0; TaskIndex < TaskList.Num(); ++TaskIndex)
				{
					if (TaskList[TaskIndex]->IsFinished())
					{
						++NumFinished;
					}
				}

				return ((float)NumFinished / TaskList.Num());
			}
		}

		return 0.0f;
	}

	// Callback for getting the text of the progress text box.
	FText HandleProgressTextBlockText( ) const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid())
		{
			if ((LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy) ||
				(LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceling))
			{
				return LOCTEXT("OperationInProgressText", "Operation in progress...");
			}

			int32 NumCanceled = 0;
			int32 NumCompleted = 0;
			int32 NumFailed = 0;

			for (int32 TaskIndex = 0; TaskIndex < TaskList.Num(); ++TaskIndex)
			{
				ELauncherTaskStatus::Type TaskStatus = TaskList[TaskIndex]->GetStatus();

				if (TaskStatus == ELauncherTaskStatus::Canceled)
				{
					++NumCanceled;
				}
				else if (TaskStatus == ELauncherTaskStatus::Completed)
				{
					++NumCompleted;
				}
				else if (TaskStatus == ELauncherTaskStatus::Failed)
				{
					++NumFailed;
				}
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("NumCompleted"), NumCompleted);
			Args.Add(TEXT("NumFailed"), NumFailed);
			Args.Add(TEXT("NumCanceled"), NumCanceled);

			return FText::Format(LOCTEXT("TasksFinishedFormatText", "Operation finished. Completed: {NumCompleted}, Failed: {NumFailed}, Canceled: {NumCanceled}"), Args);
		}

		return FText::GetEmpty();
	}

	// Callback for generating a row in the task list view.
	TSharedRef<ITableRow> HandleTaskListViewGenerateRow( ILauncherTaskPtr InItem, const TSharedRef<STableViewBase>& OwnerTable ) const
	{
		return SNew(SProjectLauncherTaskListRow)
			.Task(InItem)
			.OwnerTableView(OwnerTable);
	}

	// Callback for generating a row in the task list view.
	TSharedRef<ITableRow> HandleMessageListViewGenerateRow( TSharedPtr<FProjectLauncherMessage> InItem, const TSharedRef<STableViewBase>& OwnerTable ) const
	{
		return SNew(SProjectLauncherMessageListRow, OwnerTable)
			.Message(InItem)
			.ToolTipText(InItem->Message);
	}

	FReply HandleClearButtonClicked( )
	{
		ClearLog();
		return FReply::Handled();
	}

	FReply HandleCopyButtonClicked( )
	{
		CopyLog();
		return FReply::Handled();
	}

	FReply HandleSaveButtonClicked( )
	{
		SaveLog();
		return FReply::Handled();
	}

	bool IsRerunButtonEnabled() const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid() && 
			(LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceled || LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Completed))
		{
			return true;
		}

		return false;
	}

	FReply HandleRerunButtonClicked()
	{
		if (OnRerunClicked.IsBound())
		{
			return OnRerunClicked.Execute();
		}

		return FReply::Handled();
	}

	FText GetRerunButtonToolTip() const
	{
		return LOCTEXT("RerunButtonTooltip", "Run this launch profile.");
	}

	FText GetRerunButtonText() const
	{
		return LOCTEXT("RerunButtonLabel", "Run");
	}

	bool IsDoneButtonEnabled() const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid() && LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceling)
		{
			return false;
		}
		
		return true;
	}

	FReply HandleDoneButtonClicked()
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid() && LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy)
		{
			LauncherWorkerPtr->Cancel();
		}
		else if (LauncherWorkerPtr.IsValid() && LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceling)
		{
			// do nothing
		}
		else if (OnCloseClicked.IsBound())
		{
			return OnCloseClicked.Execute();
		}

		return FReply::Handled();
	}

	FText GetDoneButtonToolTip() const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid())
		{
			if (LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy)
			{
				return LOCTEXT("DoneButtonCancelTooltip", "Cancel the run of this profile.");
			}
			else if (LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceling)
			{
				return LOCTEXT("DoneButtonCancellingTooltip", "Currently canceling.");
			}
		}
		return LOCTEXT("DoneButtonCloseTooltip", "Close this page.");
	}
	FText GetDoneButtonText() const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();

		if (LauncherWorkerPtr.IsValid())
		{
			if (LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy)
			{
				return LOCTEXT("DoneButtonCancelLabel", "Cancel");
			}
			else if (LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceling)
			{
				return LOCTEXT("DoneButtonCancellingLabel", "Cancelling");
			}
		}
		return LOCTEXT("DoneButtonDoneLabel", "Done");
	}
	

	void ClearLog()
	{
		MessageList.Reset();
		MaxMessageListRowWidth = 0.0;
		MessageListView->RequestListRefresh();
	}

	void CopyLog()
	{
		TArray<TSharedPtr<FProjectLauncherMessage > > SelectedItems = MessageListView->GetSelectedItems();

		if (SelectedItems.Num() > 0)
		{
			FString SelectedText;

			for( int32 Index = 0; Index < SelectedItems.Num(); ++Index )
			{
				SelectedText += SelectedItems[Index]->Message.ToString();
				SelectedText += LINE_TERMINATOR;
			}

			FPlatformApplicationMisc::ClipboardCopy( *SelectedText );
		}
	}

	void SaveLog()
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		if (DesktopPlatform != NULL)
		{
			TArray<FString> Filenames;

			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

			if (DesktopPlatform->SaveFileDialog(
				ParentWindowHandle,
				LOCTEXT("SaveLogDialogTitle", "Save Log As...").ToString(),
				LastLogFileSaveDirectory,
				TEXT("ProjectLauncher.log"),
				TEXT("Log Files (*.log)|*.log"),
				EFileDialogFlags::None,
				Filenames))
			{
				if (Filenames.Num() > 0)
				{
					FString Filename = Filenames[0];

					// keep path as default for next time
					LastLogFileSaveDirectory = FPaths::GetPath(Filename);

					// add a file extension if none was provided
					if (FPaths::GetExtension(Filename).IsEmpty())
					{
						Filename += Filename + TEXT(".log");
					}

					// save file
					FArchive* LogFile = IFileManager::Get().CreateFileWriter(*Filename);

					if (LogFile != NULL)
					{
						for( int32 Index = 0; Index < MessageList.Num(); ++Index )
						{
							FString LogEntry = MessageList[Index]->Message.ToString() + LINE_TERMINATOR;

							LogFile->Serialize(TCHAR_TO_ANSI(*LogEntry), LogEntry.Len());
						}

						LogFile->Close();

						delete LogFile;
					}
					else
					{
						FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveLogDialogFileError", "Failed to open the specified file for saving!"));
					}
				}
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveLogDialogUnsupportedError", "Saving is not supported on this platform!"));
		}
	}

	FText GetSelectedProfileNameText() const
	{
		ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();
		if (LauncherWorkerPtr.IsValid())
		{
			const ILauncherProfilePtr& ProfilePtr = LauncherWorkerPtr->GetLauncherProfile();
			if (ProfilePtr.IsValid())
			{
				return FText::FromString(ProfilePtr->GetName());
			}
		}
		return FText::GetEmpty();
	}

private:

	// Holds the launcher worker.
	TWeakPtr<ILauncherWorker> LauncherWorker;

	// Holds the output log.
	TArray<TSharedPtr<FString>> OutputList;

	// Holds the output list view.
	TSharedPtr<SListView<TSharedPtr<FString>>> OutputListView;

	// Holds the progress bar.
	TSharedPtr<SProgressBar> ProgressBar;

	// Holds the task list.
	TArray<ILauncherTaskPtr> TaskList;

	// Holds the message list.
	TArray< TSharedPtr<FProjectLauncherMessage>> MessageList;

	// Holds the filtered message list.
	TArray< TSharedPtr<FProjectLauncherMessage>> FilterMessageList;

	// Holds the pending message list.
	TArray< TSharedPtr<FProjectLauncherMessage>> PendingMessages;

	// Horizontal scroll box wrapping the message list view.
	TSharedPtr<SScrollBox> HorizontalScrollBox;

	// Holds the message list view.
	TSharedPtr<SListView<TSharedPtr<FProjectLauncherMessage>>> MessageListView;

	// Caches the size of the longest row seen in the message list. Should be cleared when message list is cleared.
	float MaxMessageListRowWidth;

	// Holds the task list view.
	TSharedPtr<SListView<ILauncherTaskPtr>> TaskListView;

	// Holds the box of task statuses.
	TSharedPtr<SVerticalBox> TaskStatusBox;

	// Critical section for updating the messages
	FCriticalSection CriticalSection;

	// Holds the directory where the log file was last saved to.
	FString LastLogFileSaveDirectory;

	// Holds the copy log button.
	TSharedPtr<SButton> CopyButton;

	// Holds the clear button.
	TSharedPtr<SButton> ClearButton;

	// Holds the save button.
	TSharedPtr<SButton> SaveButton;

	// Holds a delegate to be invoked when this panel is closed.
	FOnClicked OnCloseClicked;

	// Holds a delegate to be invoked when we want the launch profile rerun.
	FOnClicked OnRerunClicked;
};


#undef LOCTEXT_NAMESPACE
