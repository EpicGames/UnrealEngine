// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Progress/SProjectLauncherTaskListRow.h"

#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SProjectLauncherTaskListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "Duration")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SProjectLauncherTaskListRow::HandleDurationText)
			];
	}
	else if (ColumnName == "Icon")
	{
		return SNew(SOverlay)

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SThrobber)
					.Animate(SThrobber::VerticalAndOpacity)
					.NumPieces(1)
					.Visibility(this, &SProjectLauncherTaskListRow::HandleThrobberVisibility)
			]

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
					.ColorAndOpacity(this, &SProjectLauncherTaskListRow::HandleIconColorAndOpacity)
					.Image(this, &SProjectLauncherTaskListRow::HandleIconImage)
			];
	}
	else if (ColumnName == "Status")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SProjectLauncherTaskListRow::HandleStatusText)
			];
	}
	else if (ColumnName == "Task")
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid())
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(TaskPtr->GetDesc()))
				];
		}
	}
	else if (ColumnName == "Warnings")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SProjectLauncherTaskListRow::HandleWarningCounterText)
			];
	}
	else if (ColumnName == "Errors")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SProjectLauncherTaskListRow::HandleErrorCounterText)
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION
