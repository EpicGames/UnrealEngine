// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherTask.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherTaskListRow"

/**
 * Implements a row widget for the launcher's task list.
 */
class SProjectLauncherTaskListRow
	: public SMultiColumnTableRow<ILauncherTaskPtr>
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherTaskListRow) { }
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(ILauncherTaskPtr, Task)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceProxyManager The device proxy manager to use.
	 */
	void Construct( const FArguments& InArgs )
	{
		Task = InArgs._Task;

		SMultiColumnTableRow<ILauncherTaskPtr>::Construct(FSuperRowType::FArguments(), InArgs._OwnerTableView.ToSharedRef());
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:

	// Callback for getting the duration of the task.
	FText HandleDurationText( ) const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid() &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Pending &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Canceled)
		{
			return FText::AsTimespan(TaskPtr->GetDuration());
		}

		return FText::GetEmpty();
	}

	// Callback for getting the color and opacity of the status icon.
	FSlateColor HandleIconColorAndOpacity( ) const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid())
		{
			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Canceled)
			{
				return FLinearColor::Yellow;
			}

			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Completed)
			{
				return FLinearColor::Green;
			}

			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Failed)
			{
				return FLinearColor::Red;
			}
		}

		return FSlateColor::UseForeground();
	}

	// Callback for getting the status icon image.
	const FSlateBrush* HandleIconImage( ) const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid())
		{
			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Canceled)
			{
				return FAppStyle::GetBrush("Icons.Cross");
			}

			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Completed)
			{
				return FAppStyle::GetBrush("Icons.Check");
			}

			if (TaskPtr->GetStatus() == ELauncherTaskStatus::Failed)
			{
				return FAppStyle::GetBrush("Icons.Cross");
			}
		}

		return NULL;
	}

	// Callback for getting the task's status text.
	FText HandleStatusText( ) const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid())
		{
			ELauncherTaskStatus::Type TaskStatus = TaskPtr->GetStatus();

			switch (TaskStatus)
			{
			case ELauncherTaskStatus::Busy:
				{
					if (TaskPtr->IsCancelling())
					{
						return LOCTEXT("StatusCancelingText", "Canceling");
					}
					return LOCTEXT("StatusInProgressText", "Busy");
				}
			case ELauncherTaskStatus::Canceled:

				return LOCTEXT("StatusCanceledText", "Canceled");

			case ELauncherTaskStatus::Completed:

				return LOCTEXT("StatusCompletedText", "Completed");

			case ELauncherTaskStatus::Failed:

				return LOCTEXT("StatusFailedText", "Failed");

			case ELauncherTaskStatus::Pending:

				return LOCTEXT("StatusPendingText", "Pending");
			}
		}

		return FText::GetEmpty();
	}

	// Callback for determining the throbber's visibility.
	EVisibility HandleThrobberVisibility( ) const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();

		if (TaskPtr.IsValid())
		{
			if ((TaskPtr->GetStatus() == ELauncherTaskStatus::Busy) ||
				(TaskPtr->IsCancelling() && TaskPtr->GetStatus() != ELauncherTaskStatus::Completed))
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Hidden;
	}

	FText HandleWarningCounterText() const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();
		if (TaskPtr.IsValid() &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Pending &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Canceled)
		{
			return FText::AsNumber(TaskPtr->GetWarningCount());
		}
		return FText::GetEmpty();
	}

	FText HandleErrorCounterText() const
	{
		ILauncherTaskPtr TaskPtr = Task.Pin();
		if (TaskPtr.IsValid() &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Pending &&
			TaskPtr->GetStatus() != ELauncherTaskStatus::Canceled)
		{
			return FText::AsNumber(TaskPtr->GetErrorCount());
		}
		return FText::GetEmpty();
	}

private:

	// Holds a pointer to the task that is displayed in this row.
	TWeakPtr<ILauncherTask> Task;
};


#undef LOCTEXT_NAMESPACE
