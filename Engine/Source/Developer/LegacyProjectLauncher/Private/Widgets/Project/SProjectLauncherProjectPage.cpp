// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Project/SProjectLauncherProjectPage.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Project/SProjectLauncherProjectPicker.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherProjectPage"


/* SProjectLauncherProjectPage interface
 *****************************************************************************/

void SProjectLauncherProjectPage::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowConfig)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WhichProjectToUseText", "Which project would you like to use?"))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0, 8.0, 0.0, 0.0)
		[
			// project loading area
			SNew(SProjectLauncherProjectPicker, InModel)
			.LaunchProfile(InArgs._LaunchProfile)
		]
	];
}


#undef LOCTEXT_NAMESPACE
