// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

/**
 * Implements the profile page for the session launcher wizard.
 */
class SProjectLauncherProjectPage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProjectPage) { }
		SLATE_ATTRIBUTE(ILauncherProfilePtr, LaunchProfile)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowConfig = true );

private:

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;
};
