// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Project/SProjectLauncherProjectPicker.h"

#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#include "Widgets/Shared/SProjectLauncherFormLabel.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherUnrealProjectPicker"


/* SProjectLauncherProjectPicker structors
 *****************************************************************************/

SProjectLauncherProjectPicker::~SProjectLauncherProjectPicker()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


/* SProjectLauncherProjectPicker interface
 *****************************************************************************/

void SProjectLauncherProjectPicker::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	LaunchProfileAttr = InArgs._LaunchProfile;

	ChildSlot
	[
		MakeProjectWidget()
	];
}


/* SProjectLauncherProjectPicker implementation
 *****************************************************************************/

TSharedRef<SWidget> SProjectLauncherProjectPicker::MakeProjectMenuWidget()
{
	FMenuBuilder MenuBuilder(true, NULL);
	{
		if (LaunchProfileAttr.IsBound())
		{
			FUIAction AnyProjectAction(FExecuteAction::CreateSP(this, &SProjectLauncherProjectPicker::HandleAnyProjectClicked, FString("Any")));
			MenuBuilder.AddMenuEntry(LOCTEXT("AnyProjectAction", "Any Project"), LOCTEXT("AnyProjectActionHint", "This profile can be used on any project."), FSlateIcon(), AnyProjectAction);
		}

		const TArray<FString>& AvailableGames = FGameProjectHelper::GetAvailableGames();
		for (int32 Index = 0; Index < AvailableGames.Num(); ++Index)
		{
			FString ProjectPath = FPaths::RootDir() / AvailableGames[Index] / AvailableGames[Index] + TEXT(".uproject");
			FUIAction ProjectAction(FExecuteAction::CreateSP(this, &SProjectLauncherProjectPicker::HandleProjectMenuEntryClicked, ProjectPath));
			MenuBuilder.AddMenuEntry(FText::FromString(AvailableGames[Index]), FText::FromString(ProjectPath), FSlateIcon(), ProjectAction);
		}

		MenuBuilder.AddMenuSeparator();

		FUIAction BrowseAction(FExecuteAction::CreateSP(this, &SProjectLauncherProjectPicker::HandleProjectMenuEntryClicked, FString()));
		MenuBuilder.AddMenuEntry(LOCTEXT("BrowseAction", "Browse..."), LOCTEXT("BrowseActionHint", "Browse for a project on your computer"), FSlateIcon(), BrowseAction);
	}

	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SProjectLauncherProjectPicker::MakeProjectWidget()
{
	TSharedRef<SWidget> Widget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.Padding(16, 0, 12, 0)
				.AutoWidth()
				[
					SNew(SProjectLauncherFormLabel)
					.ErrorToolTipText(NSLOCTEXT("ProjectLauncherBuildValidation", "NoProjectSelectedError", "A Project must be selected."))
					.ErrorVisibility(this, &SProjectLauncherProjectPicker::HandleValidationErrorIconVisibility, ELauncherProfileValidationErrors::NoProjectSelected)
					.LabelText(LOCTEXT("ProjectComboBoxLabel", "Project"))
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					// project selector
					SNew(SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SProjectLauncherProjectPicker::HandleProjectComboButtonText)
					]
					.ContentPadding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
					.MenuContent()
					[
						MakeProjectMenuWidget()
					]
					.ToolTipText(this, &SProjectLauncherProjectPicker::HandleProjectComboButtonToolTip)
				]
		];

	return Widget;
}


/* SProjectLauncherProjectPicker callbacks
 *****************************************************************************/

FText SProjectLauncherProjectPicker::HandleProjectComboButtonText() const
{
	if (LaunchProfileAttr.IsBound())
	{
		ILauncherProfilePtr Profile = LaunchProfileAttr.Get();
		if (Profile.IsValid() && Profile->HasProjectSpecified())
		{
			return FText::FromString(Profile->GetProjectName());
		}
		return LOCTEXT("AnyProjectAction", "Any Project");
	}

	FString Name = Model->GetProfileManager()->GetProjectName();
	if (!Name.IsEmpty())
	{
		return FText::FromString(Name);
	}

	return LOCTEXT("SelectProjectText", "Select...");
}


FText SProjectLauncherProjectPicker::HandleProjectComboButtonToolTip() const
{
	return LOCTEXT("SelectProjectText_Tooltip", "Select or browse for a project");
}


void SProjectLauncherProjectPicker::HandleAnyProjectClicked(FString ProjectPath)
{
	if (LaunchProfileAttr.IsBound())
	{
		ILauncherProfilePtr Profile = LaunchProfileAttr.Get();
		if (Profile.IsValid())
		{
			Profile->SetProjectSpecified(false);
		}
	}
}


void SProjectLauncherProjectPicker::HandleProjectMenuEntryClicked(FString ProjectPath)
{
	if (ProjectPath.IsEmpty())
	{
		FString DefaultPath = FPaths::RootDir();

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;
		
		TArray<FString> OutFiles;
		if (FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, LOCTEXT("SelectProjectDialogTitle", "Select a project").ToString(), DefaultPath, TEXT(""), TEXT("Project files (*.uproject)|*.uproject"), EFileDialogFlags::None, OutFiles))
		{
			SetProjectPath(OutFiles[0]);
		}
	}
	else
	{
		SetProjectPath(ProjectPath);
	}
}


void SProjectLauncherProjectPicker::SetProjectPath(FString ProjectPath)
{
	if (LaunchProfileAttr.IsBound())
	{
		ILauncherProfilePtr Profile = LaunchProfileAttr.Get();
		if (Profile.IsValid())
		{
			Profile->SetProjectSpecified(true);
			Profile->SetProjectPath(ProjectPath);
		}
	}
	else
	{
		Model->GetProfileManager()->SetProjectPath(ProjectPath);
	}
}


EVisibility SProjectLauncherProjectPicker::HandleValidationErrorIconVisibility(ELauncherProfileValidationErrors::Type Error) const
{
	// Profiles are always valid
	if (LaunchProfileAttr.IsBound())
	{
		return EVisibility::Hidden;
	}

	ILauncherProfileManagerRef ProfileManager = Model->GetProfileManager();
	if (ProfileManager->GetProjectName().IsEmpty())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}


#undef LOCTEXT_NAMESPACE
