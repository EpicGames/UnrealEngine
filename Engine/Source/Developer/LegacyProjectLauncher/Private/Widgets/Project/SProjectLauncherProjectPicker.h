// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"

class Error;

/**
 * Implements the project loading area for the session launcher wizard.
 */
class SProjectLauncherProjectPicker
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProjectPicker) { }
		SLATE_ATTRIBUTE(ILauncherProfilePtr, LaunchProfile)
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherProjectPicker( );

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/**
	 * Creates the widget for the project menu.
	 *
	 * @return The widget.
	 */
	TSharedRef<SWidget> MakeProjectMenuWidget( );

	/**
	 * Creates the widget for the project selection.
	 *
	 * @return The widget.
	 */
	TSharedRef<SWidget> MakeProjectWidget( );

private:

	// Handles getting the text for the project combo button.
	FText HandleProjectComboButtonText( ) const;

	// Handles getting the tooltip for the project combo button.
	FText HandleProjectComboButtonToolTip( ) const;

	// Handles clicking the "any project" option.
	void HandleAnyProjectClicked(FString ProjectPath);

	// Handles clicking a project menu entry.
	void HandleProjectMenuEntryClicked( FString ProjectPath );

	// Handles determining the visibility of a validation error icon.
	EVisibility HandleValidationErrorIconVisibility( ELauncherProfileValidationErrors::Type Error ) const;

	// Sets the project in the appropriate place (profile if provided otherwise on the model)
	void SetProjectPath(FString ProjectPath);

private:

	// Attribute for the launch profile this widget edits, if null it edits the project in the launcher model
	TAttribute<ILauncherProfilePtr> LaunchProfileAttr;

	// Holds the list of available projects.
	TArray<TSharedPtr<FString>> ProjectList;

	// Holds a pointer to the data model.
	TSharedPtr<FProjectLauncherModel> Model;

};
