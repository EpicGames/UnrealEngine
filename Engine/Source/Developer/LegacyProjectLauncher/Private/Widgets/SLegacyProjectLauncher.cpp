// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLegacyProjectLauncher.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#include "Models/ProjectLauncherCommands.h"
#include "Widgets/Deploy/SProjectLauncherSimpleDeviceListView.h"
#include "Widgets/Profile/SProjectLauncherProfileListView.h"
#include "Widgets/Progress/SProjectLauncherProgress.h"
#include "Widgets/Project/SProjectLauncherProjectPicker.h"
#include "Widgets/Shared/SProjectLauncherBuildTargetSelector.h"
#include "Widgets/Settings/SProjectLauncherSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "SLegacyProjectLauncher"


/* SLegacyProjectLauncher structors
 *****************************************************************************/

SLegacyProjectLauncher::SLegacyProjectLauncher()
	:bAdvanced(false)
{
	if (GConfig != NULL)
	{
		GConfig->GetBool(TEXT("FProjectLauncher"), TEXT("AdvancedMode"), bAdvanced, GEngineIni);
	}
}


SLegacyProjectLauncher::~SLegacyProjectLauncher()
{
	if (GConfig != NULL)
	{
		GConfig->SetBool(TEXT("FProjectLauncher"), TEXT("AdvancedMode"), bAdvanced, GEngineIni);
	}

	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
		FPlatformProcess::Sleep(0.5f);
	}
}


/* SLegacyProjectLauncher interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SLegacyProjectLauncher::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow, const TSharedRef<FProjectLauncherModel>& InModel)
{
	FProjectLauncherCommands::Register();

	Model = InModel;

	// create & initialize main menu bar
	TSharedRef<FWorkspaceItem> RootMenuGroup = FWorkspaceItem::NewGroup(LOCTEXT("RootMenuGroup", "Root"));

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SLegacyProjectLauncher::FillWindowMenu, RootMenuGroup),
		"Window"
	);

	ChildSlot
	[
		SAssignNew(WidgetSwitcher, SWidgetSwitcher)
		.WidgetIndex((int32)ELauncherPanels::Launch)
		
		// Empty Panel
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBorder)
		]

		// SLegacyProjectLauncher Panel
		+ SWidgetSwitcher::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)

			// Simple SLegacyProjectLauncher
			+ SSplitter::Slot()
			.Value(0.55f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
					[
						SNew(SHorizontalBox)

						// Project Bar
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 4)
						[
							SNew(SProjectLauncherProjectPicker, InModel)
						]

						// Build Target
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0,4)
						[
							SNew(SProjectLauncherBuildTargetSelector, InModel)
							.UseProfile(false)
						]

						// Advanced Button
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(6, 0, 0, 0)
						[
							
							SNew(SCheckBox)
							.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
							.IsFocusable(true)
							.ToolTipText(LOCTEXT("ToggleAdvancedOptionsToolTipText", "Toggles Advanced Options"))
							.OnCheckStateChanged(this, &SLegacyProjectLauncher::OnAdvancedChanged)
							.IsChecked(this, &SLegacyProjectLauncher::OnIsAdvanced)
							[
								SNew(SHorizontalBox)
								// Icon
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(this, &SLegacyProjectLauncher::GetAdvancedToggleBrush)
									.DesiredSizeOverride(FVector2D(16, 16))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]

								// Text
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4, 0, 4, 0)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("AdvancedButton", "Show Advanced"))
								]
							]
						]
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					[
						SAssignNew(LaunchList, SProjectLauncherSimpleDeviceListView, InModel)
						.OnProfileRun(this, &SLegacyProjectLauncher::OnProfileRun)
						.IsAdvanced(this, &SLegacyProjectLauncher::GetIsAdvanced)
					]
				]
			]

			+ SSplitter::Slot()
			.Value(0.45f)
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.FillWidth(1.0f)
							.Padding(14, 0, 0, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ProjectLauncherCustomProfilesTitle", "Custom Launch Profiles"))
								.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
							]

							

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.AutoWidth()
							.Padding(0, 0, 16, 0)
							[
								SNew(SPositiveActionButton)
								.Text(LOCTEXT("AddButtonLabel", "Add"))
								.ToolTipText(LOCTEXT("AddFilterToolTip", "Add a new custom launch profile using wizard"))
								.OnGetMenuContent(this, &SLegacyProjectLauncher::MakeProfileWizardsMenu)
							]
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(2)
					[
						SAssignNew(ProfileList, SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.Padding(0)
						[
							// Simple Launch List
							SNew(SProjectLauncherProfileListView, InModel)
							.OnProfileEdit(this, &SLegacyProjectLauncher::OnProfileEdit)
							.OnProfileRun(this, &SLegacyProjectLauncher::OnProfileRun)
							.OnProfileDelete(this, &SLegacyProjectLauncher::OnProfileDelete)
						]
					]
				]
			]
		]
		
		// Launch Settings
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(ProfileSettingsPanel, SProjectLauncherSettings, InModel)
			.OnCloseClicked(this, &SLegacyProjectLauncher::OnProfileSettingsClose)
			.OnDeleteClicked(this, &SLegacyProjectLauncher::OnProfileDelete)
		]

		// Progress Panel
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(ProgressPanel, SProjectLauncherProgress)
			.OnCloseClicked(this, &SLegacyProjectLauncher::OnProgressClose)
			.OnRerunClicked(this, &SLegacyProjectLauncher::OnRerunClicked)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SLegacyProjectLauncher implementation
*****************************************************************************/

void SLegacyProjectLauncher::FillWindowMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWorkspaceItem> RootMenuGroup)
{
#if !WITH_EDITOR
	MenuBuilder.BeginSection("WindowGlobalTabSpawners", LOCTEXT("UfeMenuGroup", "Unreal Frontend"));
	{
		FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(MenuBuilder, RootMenuGroup);
	}
	MenuBuilder.EndSection();
#endif //!WITH_EDITOR
}


/* SLegacyProjectLauncher callbacks
*****************************************************************************/

void SLegacyProjectLauncher::OnAdvancedChanged(const ECheckBoxState NewCheckedState)
{
	bAdvanced = (NewCheckedState == ECheckBoxState::Checked);
}


ECheckBoxState SLegacyProjectLauncher::OnIsAdvanced() const
{
	return (bAdvanced) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


const FSlateBrush* SLegacyProjectLauncher::GetAdvancedToggleBrush() const
{
	return FAppStyle::Get().GetBrush("Icons.Advanced");
}


bool SLegacyProjectLauncher::GetIsAdvanced() const
{
	return bAdvanced;
}


void SLegacyProjectLauncher::OnProfileEdit(const ILauncherProfileRef& Profile)
{
	Model->SelectProfile(Profile);
	WidgetSwitcher->SetActiveWidgetIndex(ELauncherPanels::ProfileEditor);
}


void SLegacyProjectLauncher::OnProfileRun(const ILauncherProfileRef& Profile)
{
	LauncherProfile = Profile;
	LauncherWorker = Model->GetSProjectLauncher()->Launch(Model->GetDeviceProxyManager(), Profile);
	
	if (LauncherWorker.IsValid())
	{
		ProgressPanel->SetLauncherWorker(LauncherWorker.ToSharedRef());
		WidgetSwitcher->SetActiveWidgetIndex(ELauncherPanels::Progress);
	}
}


void SLegacyProjectLauncher::OnProfileDelete(const ILauncherProfileRef& Profile)
{
	Model->GetProfileManager()->RemoveProfile(Profile);
}


void SLegacyProjectLauncher::OnAddCustomLaunchProfileClicked()
{
	ILauncherProfileRef Profile = Model->GetProfileManager()->AddNewProfile();
	
	OnProfileEdit(Profile);

	ProfileSettingsPanel->EnterEditMode();
}


TSharedRef<SWidget> SLegacyProjectLauncher::MakeProfileWizardsMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection("Create", LOCTEXT("CreateSection", "CREATE"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomProfileLabel", "Create Custom Profile"),
		LOCTEXT("CustomProfileDescription", "Add a new custom launch profile."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SLegacyProjectLauncher::OnAddCustomLaunchProfileClicked),
			FCanExecuteAction()
		)
	);

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("CreateFromPreset", LOCTEXT("CreateFromPreset", "CREATE FROM PRESET"));

	const TArray<ILauncherProfileWizardPtr>& Wizards = Model->GetProfileManager()->GetProfileWizards();
	for (const ILauncherProfileWizardPtr& Wizard : Wizards)
	{
		FText WizardName = Wizard->GetName();
		FText WizardDescription = Wizard->GetDescription();
				
		MenuBuilder.AddMenuEntry(
			WizardName,
			WizardDescription,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLegacyProjectLauncher::ExecProfileWizard, Wizard),
				FCanExecuteAction()
				)
			);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void SLegacyProjectLauncher::ExecProfileWizard(ILauncherProfileWizardPtr InWizard)
{
	InWizard->HandleCreateLauncherProfile(Model->GetProfileManager());
}


FReply SLegacyProjectLauncher::OnProfileSettingsClose()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
	LauncherProfile.Reset();

	WidgetSwitcher->SetActiveWidgetIndex(ELauncherPanels::Launch);

	return FReply::Handled();
}


FReply SLegacyProjectLauncher::OnProgressClose()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
	LauncherProfile.Reset();

	WidgetSwitcher->SetActiveWidgetIndex(ELauncherPanels::Launch);

	return FReply::Handled();
}


FReply SLegacyProjectLauncher::OnRerunClicked()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
	LauncherWorker = Model->GetSProjectLauncher()->Launch(Model->GetDeviceProxyManager(), LauncherProfile.ToSharedRef());

	if (LauncherWorker.IsValid())
	{
		ProgressPanel->SetLauncherWorker(LauncherWorker.ToSharedRef());
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
