// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Models/LegacyProjectLauncherModel.h"

class FMenuBuilder;
class SBorder;
class SProjectLauncherProgress;
class SProjectLauncherSettings;
class SProjectLauncherSimpleDeviceListView;
class SWindow;

struct FSlateBrush;

enum class ECheckBoxState : uint8;


/**
 * Implements a Slate widget for the launcher user interface.
 */
class SLegacyProjectLauncher
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLegacyProjectLauncher) { }

		/** Exposes a delegate to be invoked when the launcher has closed. */
		SLATE_EVENT(FSimpleDelegate, OnClosed)

	SLATE_END_ARGS()

public:

	/** Constructor. */
	SLegacyProjectLauncher();

	/** Destructor. */
	~SLegacyProjectLauncher();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 * @param InModel The view model to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow, const TSharedRef<FProjectLauncherModel>& InModel);

protected:

	/**
	 * Fills the Window menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param RootMenuGroup The root menu group.
	 */
	static void FillWindowMenu(FMenuBuilder& MenuBuilder, TSharedRef<FWorkspaceItem> RootMenuGroup);

private:

	/** Callback for toggling the advanced mode. */
	void OnAdvancedChanged(const ECheckBoxState NewCheckedState);

	/** Callback to determine if we are in advanced mode. */
	ECheckBoxState OnIsAdvanced() const;

	/** Get advanced toggle button brush. */
	const FSlateBrush* GetAdvancedToggleBrush() const;

	/** Callback for whether advanced options should be shown. */
	bool GetIsAdvanced() const;

	/** Callback for editing a profile. */
	void OnProfileEdit(const ILauncherProfileRef& Profile);

	/** Callback for running a profile. */
	void OnProfileRun(const ILauncherProfileRef& Profile);

	/** Callback for deleting a profile. */
	void OnProfileDelete(const ILauncherProfileRef& Profile);

	/** Callback for clicking the Add New Launch Profile. */
	void OnAddCustomLaunchProfileClicked();

	/** Callback for filling profile wizard menu. */
	TSharedRef<SWidget> MakeProfileWizardsMenu();

	/** Execute specified profile wizard. */
	void ExecProfileWizard(ILauncherProfileWizardPtr InWizard);

	/** Callback for when the settings panel is closed. */
	FReply OnProfileSettingsClose();

	/** Callback for when the progress panel is closed. */
	FReply OnProgressClose();

	/** Callback for when the progress panel requests the profile to be re run. */
	FReply OnRerunClicked();

private:

	/** The current launcher worker, if any. */
	ILauncherWorkerPtr LauncherWorker;

	/** The Launcher profile the launcherWorker is running. */
	ILauncherProfilePtr LauncherProfile;

	/** Holds a pointer to the view model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** The profile settings panel. */
	TSharedPtr<SProjectLauncherSettings> ProfileSettingsPanel;

	/** The progress panel. */
	TSharedPtr<SProjectLauncherProgress> ProgressPanel;

	/** The widget switcher. */
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;

	/** Contains the launch list widgets. */
//	TSharedPtr<SBorder> LaunchList;

	TSharedPtr<SProjectLauncherSimpleDeviceListView> LaunchList;

	/** Contains the profile list widgets. */
	TSharedPtr<SBorder> ProfileList;

	/** Whether we are showing advanced options. */
	bool bAdvanced;
};
