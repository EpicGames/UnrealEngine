// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Settings/SProjectLauncherSettings.h"

#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include "Models/ProjectLauncherCommands.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Widgets/Archive/SProjectLauncherArchivePage.h"
#include "Widgets/Build/SProjectLauncherBuildPage.h"
#include "Widgets/Cook/SProjectLauncherCookPage.h"
#include "Widgets/Deploy/SProjectLauncherDeployPage.h"
#include "Widgets/Launch/SProjectLauncherLaunchPage.h"
#include "Widgets/Package/SProjectLauncherPackagePage.h"
#include "Widgets/Project/SProjectLauncherProjectPage.h"
#include "Widgets/Shared/SProjectLauncherProfileNameDescEditor.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherSettings"


/* SProjectLauncherSettings structors
 *****************************************************************************/

SProjectLauncherSettings::SProjectLauncherSettings()
	: CommandList(new FUICommandList())
{ }


SProjectLauncherSettings::~SProjectLauncherSettings()
{
}


/* SProjectLauncherSettings interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncherSettings::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;

	OnCloseClicked = InArgs._OnCloseClicked;
	OnDeleteClicked = InArgs._OnDeleteClicked;
	
	CreateCommands();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SBorder)
					.Padding(2)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							.Padding(4.0f, 0.0f, 4.0f, 0.0f)
							[
								SNew(SProjectLauncherProfileNameDescEditor, InModel, true)
									.LaunchProfile(this, &SProjectLauncherSettings::GetLaunchProfile)
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.0f, 0.0f, 0.0f, 0.0f)
							.HAlign(HAlign_Right)
							[
								MakeToolbar(CommandList)
							]
					]
			]

		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		[
			SNew(SScrollBox)
			.Visibility(this, &SProjectLauncherSettings::HandleSettingsScrollBoxVisibility)

			+ SScrollBox::Slot()
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SGridPanel)
				.FillColumn(1, 1.0f)

				// project section
				+ SGridPanel::Slot(0, 0)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("ProjectSectionHeader", "Project"))
				]

				+ SGridPanel::Slot(1, 0)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherProjectPage, InModel)
					.LaunchProfile(this, &SProjectLauncherSettings::GetLaunchProfile)
				]

				// build section
				+ SGridPanel::Slot(0, 1)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 2)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("BuildSectionHeader", "Build"))
				]

				+ SGridPanel::Slot(1, 2)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherBuildPage, InModel)
				]

				// cook section
				+ SGridPanel::Slot(0, 3)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 4)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("CookSectionHeader", "Cook"))
				]

				+ SGridPanel::Slot(1, 4)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherCookPage, InModel)
				]

				// package section
				+ SGridPanel::Slot(0, 5)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 6)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("PackageSectionHeader", "Package"))
				]

				+ SGridPanel::Slot(1, 6)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherPackagePage, InModel)
				]

				// archive section
				+ SGridPanel::Slot(0, 7)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 8)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("ArchiveSectionHeader", "Archive"))
				]

				+ SGridPanel::Slot(1, 8)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherArchivePage, InModel)
				]
				
				// deploy section
				+ SGridPanel::Slot(0, 9)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 10)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("DeploySectionHeader", "Deploy"))
				]

				+ SGridPanel::Slot(1, 10)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherDeployPage, InModel)
				]

				// launch section
				+ SGridPanel::Slot(0, 11)
				.ColumnSpan(3)
				.Padding(0.0f, 16.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SGridPanel::Slot(0, 12)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Bold", 13))
					.Text(LOCTEXT("LaunchSectionHeader", "Launch"))
				]

				+ SGridPanel::Slot(1, 12)
				.HAlign(HAlign_Fill)
				.Padding(32.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SProjectLauncherLaunchPage, InModel)
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SProjectLauncherSettings::EnterEditMode()
{
	if (NameEditBox.IsValid())
	{
		NameEditBox->EnterEditingMode();
	}
}


void SProjectLauncherSettings::CreateCommands()
{
	const FProjectLauncherCommands& Commands = FProjectLauncherCommands::Get();

	// Close command
	CommandList->MapAction(Commands.CloseSettings,
		FExecuteAction::CreateRaw(this, &SProjectLauncherSettings::HandleCloseActionExecute),
		FCanExecuteAction::CreateRaw(this, &SProjectLauncherSettings::HandleCloseActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SProjectLauncherSettings::HandleCloseActionIsChecked)
		);

	CommandList->MapAction(Commands.DeleteProfile,
		FExecuteAction::CreateRaw(this, &SProjectLauncherSettings::HandleDeleteActionExecute),
		FCanExecuteAction::CreateRaw(this, &SProjectLauncherSettings::HandleDeleteActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SProjectLauncherSettings::HandleDeleteActionIsChecked)
		);
}


TSharedRef<SWidget> SProjectLauncherSettings::MakeToolbar(const TSharedRef<FUICommandList>& InCommandList)
{
	FToolBarBuilder ToolBarBuilder(InCommandList, FMultiBoxCustomization::None);

	ToolBarBuilder.BeginSection("Tasks");
	{
		ToolBarBuilder.AddToolBarButton(FProjectLauncherCommands::Get().CloseSettings, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Previous")));
	}

	return ToolBarBuilder.MakeWidget();
}


/* SProjectLauncherSettings callbacks
*****************************************************************************/

ILauncherProfilePtr SProjectLauncherSettings::GetLaunchProfile() const
{
	return Model->GetSelectedProfile();
}


EVisibility SProjectLauncherSettings::HandleSelectProfileTextBlockVisibility() const
{
	if (Model->GetSelectedProfile().IsValid())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}


EVisibility SProjectLauncherSettings::HandleSettingsScrollBoxVisibility() const
{
	if (Model->GetSelectedProfile().IsValid())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


FText SProjectLauncherSettings::OnGetNameText() const
{
	const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
	if (LaunchProfile.IsValid())
	{
		return FText::FromString(LaunchProfile->GetName());
	}
	return FText();
}


void SProjectLauncherSettings::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
	if (LaunchProfile.IsValid())
	{
		Model->GetProfileManager()->ChangeProfileName(LaunchProfile.ToSharedRef(), NewText.ToString());
	}
}


FText SProjectLauncherSettings::OnGetDescriptionText() const
{
	const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
	if (LaunchProfile.IsValid())
	{
		FString Desc = LaunchProfile->GetDescription();
		if (!Desc.IsEmpty())
		{
			return FText::FromString(Desc);
		}
	}
	return FText(LOCTEXT("LaunchProfileEnterDescription", "Enter a description here."));
}


void SProjectLauncherSettings::OnDescriptionTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
	if (LaunchProfile.IsValid())
	{
		LaunchProfile->SetDescription(NewText.ToString());
	}
}


void SProjectLauncherSettings::HandleCloseActionExecute()
{
	const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
	if (LaunchProfile.IsValid())
	{
		Model->GetProfileManager()->SaveJSONProfile(LaunchProfile.ToSharedRef());
		//@Todo: FIX! Very Heavy Handed! Will have to re factor the device group saving code, but branch is tonight and this is safe. 
		Model->GetProfileManager()->SaveDeviceGroups();
	}

	if (OnCloseClicked.IsBound())
	{
		OnCloseClicked.Execute();
	}
}


bool SProjectLauncherSettings::HandleCloseActionIsChecked() const
{
	return false;
}


bool SProjectLauncherSettings::HandleCloseActionCanExecute() const
{
	return true;
}


void SProjectLauncherSettings::HandleDeleteActionExecute()
{
	if (OnDeleteClicked.IsBound())
	{
		const ILauncherProfilePtr& LaunchProfile = Model->GetSelectedProfile();
		if (LaunchProfile.IsValid())
		{
			OnDeleteClicked.Execute(LaunchProfile.ToSharedRef());
		}

		if (OnCloseClicked.IsBound())
		{
			OnCloseClicked.Execute();
		}
	}
}


bool SProjectLauncherSettings::HandleDeleteActionIsChecked() const
{
	return false;
}


bool SProjectLauncherSettings::HandleDeleteActionCanExecute() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
