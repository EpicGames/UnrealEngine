// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Framework/SlateDelegates.h"
#include "Layout/Visibility.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Widgets/Shared/ProjectLauncherDelegates.h"

class FProjectLauncherModel;
class FUICommandList;
class SWidget;


namespace ELauncherWizardPages
{
	/**
	 * Enumerates the session launcher wizard pages.
	 */
	enum Type
	{
		/** The 'Build' page. */
		BuildPage,

		/** The 'Cook' page. */
		CookPage,

		/** The 'Package' page. */
		PackagePage,

		/** The 'Deploy' page. */
		DeployPage,

		/** The 'Launch' page. */
		LaunchPage,

		/** The 'Preview' page. */
		PreviewPage
	};
}


/**
 * Implements the launcher settings widget.
 */
class SProjectLauncherSettings
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherSettings) { }
	SLATE_EVENT(FOnClicked, OnCloseClicked)
	SLATE_EVENT(FOnProfileRun, OnDeleteClicked)
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SProjectLauncherSettings();

	/** Destructor. */
	~SProjectLauncherSettings();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);

	void EnterEditMode();

private:

	/** Create the menu command list. */
	void CreateCommands();

	/** Make a Toolbar using the command list. */
	TSharedRef<SWidget> MakeToolbar(const TSharedRef<FUICommandList>& InCommandList);

private:

	/** Callback for getting the launch profile we are currently editing. */
	TSharedPtr<ILauncherProfile> GetLaunchProfile()const;

	/** Callback for getting the visibility of the 'Select Profile' text block. */
	EVisibility HandleSelectProfileTextBlockVisibility() const;

	/** Callback for getting the visibility of the settings scroll box. */
	EVisibility HandleSettingsScrollBoxVisibility() const;

	/** Callback to get the name of the launch profile. */
	FText OnGetNameText() const;

	/** Callback to set the name of the launch profile. */
	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	/** Callback to get the description of the launch profile. */
	FText OnGetDescriptionText() const;

	/** Callback to set the description of the launch profile. */
	void OnDescriptionTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	/** Callbacks for executing the 'Close' action. */
	void HandleCloseActionExecute();
	bool HandleCloseActionIsChecked() const;
	bool HandleCloseActionCanExecute() const;

	/** Callbacks for executing the 'Delete' action. */
	void HandleDeleteActionExecute();
	bool HandleDeleteActionIsChecked() const;
	bool HandleDeleteActionCanExecute() const;

private:

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	/** Holds the list of UI commands for the profile settings. */
	TSharedRef<FUICommandList> CommandList;

	/** Holds a delegate to be invoked when this panel is closed. */
	FOnClicked OnCloseClicked;

	/** Holds a delegate to be invoked when a rerun of the profile is requested. */
	FOnClicked OnRerunClicked;

	/** Holds a delegate to be invoked when this profile is deleted. */
	FOnProfileRun OnDeleteClicked;

	/** Hold a pointer to the launch profile name edit box. */
	TSharedPtr<SInlineEditableTextBlock> NameEditBox;
};
