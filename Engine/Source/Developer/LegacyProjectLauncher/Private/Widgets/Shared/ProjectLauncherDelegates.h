// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "ILauncherProfile.h"


/**
 * Delegate type for SLegacyProjectLauncher Profile running.
 *
 * The first parameter is the SLegacyProjectLauncher Profile that you want to run.
 */
DECLARE_DELEGATE_OneParam(FOnProfileRun, const ILauncherProfileRef&)
