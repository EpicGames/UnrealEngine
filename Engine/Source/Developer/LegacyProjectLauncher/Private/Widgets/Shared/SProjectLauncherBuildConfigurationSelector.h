// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "InstalledPlatformInfo.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherBuildConfigurationSelector"


/**
 * Delegate type for build configuration selections.
 *
 * The first parameter is the selected build configuration.
 */
DECLARE_DELEGATE_OneParam(FOnSessionSProjectLauncherBuildConfigurationSelected, EBuildConfiguration)


/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherBuildConfigurationSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherBuildConfigurationSelector) { }
		SLATE_EVENT(FOnSessionSProjectLauncherBuildConfigurationSelected, OnConfigurationSelected)
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs)
	{
		OnConfigurationSelected = InArgs._OnConfigurationSelected;

		struct FConfigInfo
		{
			FText ToolTip;
			EBuildConfiguration Configuration;
		};

		const FConfigInfo Configurations[] =
		{
			{LOCTEXT("DebugActionHint", "Debug configuration."), EBuildConfiguration::Debug},
			{LOCTEXT("DebugGameActionHint", "DebugGame configuration."), EBuildConfiguration::DebugGame},
			{LOCTEXT("DevelopmentActionHint", "Development configuration."), EBuildConfiguration::Development},
			{LOCTEXT("ShippingActionHint", "Shipping configuration."), EBuildConfiguration::Shipping},
			{LOCTEXT("TestActionHint", "Test configuration."), EBuildConfiguration::Test}
		};

		// create build configurations menu
		FMenuBuilder MenuBuilder(true, NULL);
		{
			for (const FConfigInfo& ConfigInfo : Configurations)
			{
				if (FInstalledPlatformInfo::Get().IsValidConfiguration(ConfigInfo.Configuration))
				{
					FUIAction UIAction(FExecuteAction::CreateSP(this, &SProjectLauncherBuildConfigurationSelector::HandleMenuEntryClicked, ConfigInfo.Configuration));
					MenuBuilder.AddMenuEntry(EBuildConfigurations::ToText(ConfigInfo.Configuration), ConfigInfo.ToolTip, FSlateIcon(), UIAction);
				}
			}
		}

		FSlateFontInfo TextFont = InArgs._Font.IsSet() ? InArgs._Font.Get() : FCoreStyle::Get().GetFontStyle(TEXT("SmallFont"));
		
		ChildSlot
		[
			// build configuration menu
			SNew(SComboButton)
			.VAlign(VAlign_Center)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(TextFont)
				.Text(InArgs._Text)
			]
			.ContentPadding(FMargin(6.0f, 0.0f))
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
		];
	}

private:

	// Callback for clicking a menu entry.
	void HandleMenuEntryClicked( EBuildConfiguration Configuration )
	{
		OnConfigurationSelected.ExecuteIfBound(Configuration);
	}

private:

	// Holds a delegate to be invoked when a build configuration has been selected.
	FOnSessionSProjectLauncherBuildConfigurationSelected OnConfigurationSelected;
};


#undef LOCTEXT_NAMESPACE
