// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProjectLauncherBuildTargetSelector.h"

#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Shared/SProjectLauncherFormLabel.h"



#define LOCTEXT_NAMESPACE "SProjectLauncherUnrealBuildTargetSelector"


/* SProjectLauncherBuildTargetSelector structors
 *****************************************************************************/

SProjectLauncherBuildTargetSelector::~SProjectLauncherBuildTargetSelector()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
		Model->GetProfileManager()->OnProjectChanged().RemoveAll(this);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bUseProfile)
	{
		ILauncherProfilePtr Profile = GetProfile();
		if (Profile.IsValid())
		{
			Profile->OnProjectChanged().RemoveAll(this);
			Profile->OnBuildTargetOptionsChanged().RemoveAll(this);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


/* SProjectLauncherBuildTargetSelector interface
 *****************************************************************************/

void SProjectLauncherBuildTargetSelector::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel)
{
	Model = InModel;
	bUseProfile = InArgs._UseProfile.Get();

	if (bUseProfile)
	{
		Model->OnProfileSelected().AddSP(this, &SProjectLauncherBuildTargetSelector::HandleProfileManagerProfileSelected);
	}
	Model->GetProfileManager()->OnProjectChanged().AddSP(this, &SProjectLauncherBuildTargetSelector::HandleProfileProjectChanged);
	
	FText LabelText = bUseProfile ? LOCTEXT("BuildTargetComboBoxLabel", "Build Target") : LOCTEXT("DefaultBuildTargetComboBoxLabel", "Default Build Target");

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0)
		.Visibility(this, &SProjectLauncherBuildTargetSelector::IsBuildTargetVisible)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.Padding(16, 0, 12, 0)
				.AutoWidth()
				[
					SNew(SProjectLauncherFormLabel)
					.LabelText(LabelText)
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
				SAssignNew(BuildTargetCombo,SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&BuildTargetList)
				.OnSelectionChanged( this, &SProjectLauncherBuildTargetSelector::HandleBuildTargetSelectionChanged )
				.OnGenerateWidget( this, &SProjectLauncherBuildTargetSelector::HandleBuildTargetGenerateWidget )
				.ContentPadding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SProjectLauncherBuildTargetSelector::HandleBuildTargetComboButtonText)
				]
			]
		]
	];

	RefreshBuildTargetList();
}




FText SProjectLauncherBuildTargetSelector::GetBuildTargetText( FString BuildTarget ) const
{
	if (!BuildTarget.IsEmpty())
	{
		return FText::FromString(BuildTarget);
	}

	return LOCTEXT("NoBuildTargetText", "Default");
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FText SProjectLauncherBuildTargetSelector::HandleBuildTargetComboButtonText() const
{
	FString BuildTarget;

	if (bUseProfile)
	{
		ILauncherProfilePtr Profile = GetProfile();
		if (Profile.IsValid() && Profile->HasBuildTargetSpecified())
		{
			const TArray<FString> BuildTargets = Profile->GetBuildTargets();

			BuildTarget = BuildTargets.Num() > 0 ? BuildTargets[0] : FString();
		}
	}
	else
	{
		BuildTarget = Model->GetProfileManager()->GetBuildTarget();
	}

	return GetBuildTargetText(BuildTarget);
}


void SProjectLauncherBuildTargetSelector::HandleBuildTargetSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		SetBuildTarget(*Item.Get());
	}
}

TSharedRef<SWidget> SProjectLauncherBuildTargetSelector::HandleBuildTargetGenerateWidget(TSharedPtr<FString> Item)
{
	FText BuildTargetText = GetBuildTargetText(*Item.Get());
	return SNew(STextBlock).Text(BuildTargetText);
}

EVisibility SProjectLauncherBuildTargetSelector::IsBuildTargetVisible() const
{
	if (bUseProfile)
	{
		ILauncherProfilePtr Profile = GetProfile();
		if (Profile.IsValid() && Profile->RequiresExplicitBuildTargetName())
		{
			return EVisibility::Visible;
		}
	}
	else
	{
		if (Model->GetProfileManager()->GetAllExplicitBuildTargetNames().Num() > 0)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}



void SProjectLauncherBuildTargetSelector::SetBuildTarget(const FString& BuildTarget)
{
	if (bUseProfile)
	{
		ILauncherProfilePtr Profile = GetProfile();
		if (Profile.IsValid())
		{
			Profile->ClearBuildTargets();
			if (!BuildTarget.IsEmpty())
			{
				Profile->AddBuildTarget(BuildTarget);
			}
			Profile->SetBuildTargetSpecified(true);
		}
	}
	else
	{
		Model->GetProfileManager()->SetBuildTarget(BuildTarget);
	}
}

void SProjectLauncherBuildTargetSelector::HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile)
{
	if (PreviousProfile.IsValid())
	{
		PreviousProfile->OnProjectChanged().RemoveAll(this);
		PreviousProfile->OnBuildTargetOptionsChanged().RemoveAll(this);
	}
	if (SelectedProfile.IsValid())
	{
		SelectedProfile->OnProjectChanged().AddSP(this, &SProjectLauncherBuildTargetSelector::HandleProfileProjectChanged);
		SelectedProfile->OnBuildTargetOptionsChanged().AddSP(this, &SProjectLauncherBuildTargetSelector::HandleProfileBuildTargetOptionsChanged);
	}
	RefreshBuildTargetList();
}


void SProjectLauncherBuildTargetSelector::HandleProfileProjectChanged()
{
	RefreshBuildTargetList();
}

void SProjectLauncherBuildTargetSelector::HandleProfileBuildTargetOptionsChanged()
{
	RefreshBuildTargetList();
}


void SProjectLauncherBuildTargetSelector::RefreshBuildTargetList()
{
	// make a list of all suitable build target names
	TArray<FString> ExplicitBuildTargetNames;
	if (bUseProfile)
	{
		ILauncherProfilePtr Profile = GetProfile();
		if (Profile.IsValid())
		{
			ExplicitBuildTargetNames = Profile->GetExplicitBuildTargetNames();
		}
	}
	else
	{
		ExplicitBuildTargetNames = Model->GetProfileManager()->GetAllExplicitBuildTargetNames();
	}

	BuildTargetList.Reset();
	for ( const FString& BuildTarget : ExplicitBuildTargetNames)
	{
		BuildTargetList.Add(MakeShared<FString>(BuildTarget));
	}

	// add a blank entry to allow resetting back to 'default'
	BuildTargetList.Add(MakeShared<FString>());

	// refresh UI
	if (BuildTargetCombo.IsValid())
	{
		BuildTargetCombo->RefreshOptions();
	}
}

ILauncherProfilePtr SProjectLauncherBuildTargetSelector::GetProfile() const
{
	if (ensure(bUseProfile) && Model.IsValid())
	{
		return Model->GetSelectedProfile();
	}

	return nullptr;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
