// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Models/LegacyProjectLauncherModel.h"

class Error;

DECLARE_DELEGATE_OneParam(FOnLauncherProfileChanged, ILauncherProfilePtr);

class SProjectLauncherBuildTargetSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherBuildTargetSelector)
		: _UseProfile(false)
		{}
		SLATE_ATTRIBUTE(bool, UseProfile) // whether we are showing items for the current profile or not
	SLATE_END_ARGS()

public:

	/**
	 * Destructor.
	 */
	~SProjectLauncherBuildTargetSelector();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel);


private:

	void RefreshBuildTargetList();

	EVisibility IsBuildTargetVisible() const;
	FText HandleBuildTargetComboButtonText() const;
	FText GetBuildTargetText( FString BuildTarget ) const;
	void HandleBuildTargetSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> HandleBuildTargetGenerateWidget(TSharedPtr<FString> Item);
	void HandleProfileManagerProfileSelected(const ILauncherProfilePtr& SelectedProfile, const ILauncherProfilePtr& PreviousProfile);
	void HandleProfileProjectChanged();
	void HandleProfileBuildTargetOptionsChanged();

	void SetBuildTarget(const FString& BuildTarget);

	ILauncherProfilePtr GetProfile() const;


private:

	TSharedPtr<SComboBox<TSharedPtr<FString>>> BuildTargetCombo;
	TArray<TSharedPtr<FString>> BuildTargetList;
	bool bUseProfile;
	TSharedPtr<FProjectLauncherModel> Model;

};
