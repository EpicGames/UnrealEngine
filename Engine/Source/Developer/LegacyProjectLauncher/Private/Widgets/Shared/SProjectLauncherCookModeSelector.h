// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "ILauncherProfile.h"
#include "Widgets/Text/STextBlock.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherCookModeSelector"


/**
 * Delegate type for build configuration selections.
 *
 * The first parameter is the selected build configuration.
 */
DECLARE_DELEGATE_OneParam(FOnSProjectLauncherCookModeSelected, ELauncherProfileCookModes::Type)


/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherCookModeSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherCookModeSelector) { }
		SLATE_EVENT(FOnSProjectLauncherCookModeSelected, OnCookModeSelected)
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs )
	{
		OnCookModeSelected = InArgs._OnCookModeSelected;

		// create cook modes menu
		FMenuBuilder MenuBuilder(true, NULL);
		{
			FUIAction ByTheBookAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookModeSelector::HandleMenuEntryClicked, ELauncherProfileCookModes::ByTheBook));
			MenuBuilder.AddMenuEntry(LOCTEXT("ByTheBookAction", "By the book"), LOCTEXT("ByTheBookActionHint", "Specify which content should be cooked and cook everything in advance prior to launching the game."), FSlateIcon(), ByTheBookAction);

			FUIAction OnTheFlyAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookModeSelector::HandleMenuEntryClicked, ELauncherProfileCookModes::OnTheFly));
			MenuBuilder.AddMenuEntry(LOCTEXT("OnTheFlyAction", "On the fly"), LOCTEXT("OnTheFlyActionHint", "Cook the content at run-time before it is being sent to the device."), FSlateIcon(), OnTheFlyAction);

			FUIAction DoNotCookAction(FExecuteAction::CreateSP(this, &SProjectLauncherCookModeSelector::HandleMenuEntryClicked, ELauncherProfileCookModes::DoNotCook));
			MenuBuilder.AddMenuEntry(LOCTEXT("DoNotCookAction", "Do not cook"), LOCTEXT("DoNotCookActionHint", "Do not cook the content at this time."), FSlateIcon(), DoNotCookAction);
		}

		ChildSlot
		[
			// build configuration menu
			SNew(SComboButton)
			.VAlign(VAlign_Center)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("SmallFont")))
				.Text(InArgs._Text)
			]
			.ContentPadding(FMargin(6.0f, 0.0f))
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
		];
	}

private:

	// Callback for clicking a menu entry.
	void HandleMenuEntryClicked(ELauncherProfileCookModes::Type CookMode)
	{
		OnCookModeSelected.ExecuteIfBound(CookMode);
	}

private:

	// Holds a delegate to be invoked when a build configuration has been selected.
	FOnSProjectLauncherCookModeSelected OnCookModeSelected;
};


#undef LOCTEXT_NAMESPACE
