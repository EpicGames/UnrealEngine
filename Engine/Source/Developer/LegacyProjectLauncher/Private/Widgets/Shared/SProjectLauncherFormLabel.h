// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"

/**
 * Implements a widget for form input field labels with optional validation errors.
 */
class SProjectLauncherFormLabel
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherFormLabel)
		: _ErrorToolTipText()
		, _ErrorVisibility(EVisibility::Hidden)
		, _LabelText()
	{ }

		/** The tool tip text for the validation error icon. */
		SLATE_ATTRIBUTE(FText, ErrorToolTipText)

		/** The visibility of the validation error icon. */
		SLATE_ATTRIBUTE(EVisibility, ErrorVisibility)

		/** The text of the form label. */
		SLATE_ATTRIBUTE(FText, LabelText)

	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct(	const FArguments& InArgs )
	{
		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InArgs._LabelText)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
				.ToolTipText(InArgs._ErrorToolTipText)
				.Visibility(InArgs._ErrorVisibility)
			]
		];
	}
};
