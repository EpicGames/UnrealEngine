// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "ILauncherProfileLaunchRole.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherInstanceTypeSelector"


/**
 * Delegate type for build configuration selections.
 *
 * The first parameter is the selected build configuration.
 */
DECLARE_DELEGATE_OneParam(FOnSProjectLauncherInstanceTypeSelected, ELauncherProfileRoleInstanceTypes::Type)


/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherInstanceTypeSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherInstanceTypeSelector) { }
		SLATE_EVENT(FOnSProjectLauncherInstanceTypeSelected, OnInstanceTypeSelected)
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(	const FArguments& InArgs )
	{
		OnInstanceTypeSelected = InArgs._OnInstanceTypeSelected;

		// create instance types menu
		FMenuBuilder MenuBuilder(true, NULL);
		{
			FUIAction StandaloneClientAction(FExecuteAction::CreateSP(this, &SProjectLauncherInstanceTypeSelector::HandleMenuEntryClicked, ELauncherProfileRoleInstanceTypes::StandaloneClient));
			MenuBuilder.AddMenuEntry(LOCTEXT("StandaloneClient", "Standalone Client"), LOCTEXT("StandaloneClientActionHint", "Launch this instance as a standalone game client."), FSlateIcon(), StandaloneClientAction);

			FUIAction ListenServerAction(FExecuteAction::CreateSP(this, &SProjectLauncherInstanceTypeSelector::HandleMenuEntryClicked, ELauncherProfileRoleInstanceTypes::ListenServer));
			MenuBuilder.AddMenuEntry(LOCTEXT("ListenServer", "Listen Server"), LOCTEXT("ListenServerActionHint", "Launch this instance as a game client that can accept connections from other clients."), FSlateIcon(), ListenServerAction);

			FUIAction DedicatedServerAction(FExecuteAction::CreateSP(this, &SProjectLauncherInstanceTypeSelector::HandleMenuEntryClicked, ELauncherProfileRoleInstanceTypes::DedicatedServer));
			MenuBuilder.AddMenuEntry(LOCTEXT("DedicatedServer", "Dedicated Server"), LOCTEXT("DedicatedServerActionHint", "Launch this instance as a dedicated game server."), FSlateIcon(), DedicatedServerAction);

			//FUIAction UnrealEditorAction(FExecuteAction::CreateSP(this, &SProjectLauncherInstanceTypeSelector::HandleMenuEntryClicked, ELauncherProfileRoleInstanceTypes::UnrealEditor));
			//MenuBuilder.AddMenuEntry(LOCTEXT("UnrealEditor", "Unreal Editor"), LOCTEXT("UnrealEditorActionHint", "Launch this instance as an Unreal Editor."), FSlateIcon(), UnrealEditorAction);
		}

		ChildSlot
		[
			// build configuration menu
			SNew(SComboButton)
			.VAlign(VAlign_Center)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(InArgs._Text)
			]
			.ContentPadding(FMargin(6.0f, 2.0f))
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
		];
	}

private:

	// Callback for clicking a menu entry.
	void HandleMenuEntryClicked(ELauncherProfileRoleInstanceTypes::Type InstanceType)
	{
		OnInstanceTypeSelected.ExecuteIfBound(InstanceType);
	}

private:

	// Holds a delegate to be invoked when a build configuration has been selected.
	FOnSProjectLauncherInstanceTypeSelected OnInstanceTypeSelected;
};


#undef LOCTEXT_NAMESPACE
