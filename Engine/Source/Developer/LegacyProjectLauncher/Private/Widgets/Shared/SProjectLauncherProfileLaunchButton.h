// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Shared/SProjectLauncherValidation.h"
#include "SSimpleButton.h"
#define LOCTEXT_NAMESPACE "SProjectLauncherProfileLaunchButton"


/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherProfileLaunchButton
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProfileLaunchButton) { }
	SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_ATTRIBUTE(ILauncherProfilePtr, LaunchProfile)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, bool InShowText)
	{
		LaunchProfileAttr = InArgs._LaunchProfile;

		TSharedPtr<SVerticalBox> VerticalBoxWidget;
		ChildSlot
		[	
			SNew(SSimpleButton)
			.OnClicked(InArgs._OnClicked)
			.IsEnabled(this, &SProjectLauncherProfileLaunchButton::ButtonEnabled)
			.Icon(this, &SProjectLauncherProfileLaunchButton::GetLaunchIcon)
		];

		// Add launch text is this was requested
		if (InShowText && VerticalBoxWidget.IsValid())
		{
			VerticalBoxWidget->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectLauncherLaunch", "Launch"))
			];
		}

		// Tooltip when we have validation errors
		SAssignNew(ErrorToolTipWidget, SToolTip)
		[
			SNew(SProjectLauncherValidation)
			.LaunchProfile(InArgs._LaunchProfile)
		];

		// Otherwise we fall back on simple text
		SetToolTipText(LOCTEXT("ProjectLauncherLaunchToolTip", "Launch this profile"));
	}

	virtual TSharedPtr<IToolTip> GetToolTip() override
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid() && LaunchProfile->HasValidationError())
		{
			ErrorToolTipWidget->SetContentWidget(
				SNew(SProjectLauncherValidation)
				.LaunchProfile(LaunchProfileAttr));

			return ErrorToolTipWidget;
		}
		else if (!IsLegacyProfile())
		{
			ErrorToolTipWidget->SetContentWidget(
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectLauncherNotLegacyProfile", "This profile can only be used with the new Project Launcher")));

			return ErrorToolTipWidget;
		}
		return SWidget::GetToolTip();
	}

private:

	// @returns true if there is an error in the launch profile, no launch profile is considered an error
	bool HasError() const
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid() && !LaunchProfile->HasValidationError())
		{
			return false;
		}
		return true;
	}

	bool IsLegacyProfile() const
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		return LaunchProfile.IsValid() && LaunchProfile->SupportsLegacyProjectLauncher();
	};

	// Callback to see if the launch button should be enabled
	bool ButtonEnabled() const
	{
		return !HasError() && IsLegacyProfile();
	}

	// Get the SlateIcon for Launch Button
	const FSlateBrush* GetLaunchIcon() const
	{
		return HasError() ? FAppStyle::Get().GetBrush(TEXT("Icons.Error")) : FAppStyle::Get().GetBrush("Icons.Launch");
	}

private:

	/** Attribute for the launch profile this widget launches. */
	TAttribute<ILauncherProfilePtr> LaunchProfileAttr;

	/** Holds a pointer to our custom tooltip. */
	TSharedPtr<SToolTip> ErrorToolTipWidget;

};


#undef LOCTEXT_NAMESPACE
