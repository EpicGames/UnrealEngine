// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SProjectLauncherProfileNameDescEditor.h"

#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherProfileNameDescEditor"


void SProjectLauncherProfileNameDescEditor::Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowAddDescriptionText)
{
	EnterTextDescription = FText(LOCTEXT("LaunchProfileEnterDescription", "Enter a description here."));

	Model = InModel;
	LaunchProfileAttr = InArgs._LaunchProfile;
	bShowAddDescriptionText = InShowAddDescriptionText;

	ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(23, 0, 27, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(20.f)
				.HeightOverride(20.f)
				[
					SNew(SImage)
					.Image(this, &SProjectLauncherProfileNameDescEditor::HandleProfileImage)
				]
			]

			+ SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Center)
				.Padding(0, 6, 0, 6)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 4, 2, 4)
					[
						SAssignNew(NameEditableTextBlock, SInlineEditableTextBlock)
						.Text(this, &SProjectLauncherProfileNameDescEditor::OnGetNameText)
						.OnTextCommitted(this, &SProjectLauncherProfileNameDescEditor::OnNameTextCommitted)
						.Cursor(EMouseCursor::TextEditBeam)
					]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2, 4, 2, 4)
						[
							SNew(SInlineEditableTextBlock)
							.Text(this, &SProjectLauncherProfileNameDescEditor::OnGetDescriptionText)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.OnTextCommitted(this, &SProjectLauncherProfileNameDescEditor::OnDescriptionTextCommitted)
							.Cursor(EMouseCursor::TextEditBeam)
						]
				]
		];
}


void SProjectLauncherProfileNameDescEditor::TriggerNameEdit()
{
	if (NameEditableTextBlock.IsValid())
	{
		NameEditableTextBlock->EnterEditingMode();
	}
}


#undef LOCTEXT_NAMESPACE
