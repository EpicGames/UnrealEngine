// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Models/LegacyProjectLauncherModel.h"
#include "Styling/AppStyle.h"

/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherProfileNameDescEditor
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherProfileNameDescEditor) { }
		SLATE_ATTRIBUTE(ILauncherProfilePtr, LaunchProfile)
		SLATE_ATTRIBUTE(ILauncherProfileManagerPtr, InModel)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FProjectLauncherModel>& InModel, bool InShowAddDescriptionText);

	/**
	 * Triggers a name edit for the profile.
	 */
	void TriggerNameEdit();

private:

	// Callback for getting the icon image of the profile.
	const FSlateBrush* HandleProfileImage() const
	{
		//const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(*DeviceProxy->GetTargetPlatformName(SimpleProfile->GetDeviceVariant()));
		//return (PlatformInfo) ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(PlatformInfo::EPlatformIconSize::Large)) : FStyleDefaults::GetNoBrush();
		return FAppStyle::GetBrush("LauncherCommand.QuickLaunch");
	}

	// Callback to get the name of the launch profile.
	FText OnGetNameText() const
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid())
		{
			return FText::FromString(LaunchProfile->GetName());
		}
		return FText();
	}

	// Callback to set the name of the launch profile.
	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid())
		{
			Model->GetProfileManager()->ChangeProfileName(LaunchProfile.ToSharedRef(), NewText.ToString());
		}
	}

	// Callback to get the description of the launch profile.
	FText OnGetDescriptionText() const
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid())
		{
			FString Desc = LaunchProfile->GetDescription();
			if (!bShowAddDescriptionText || !Desc.IsEmpty())
			{
				return FText::FromString(Desc);
			}
		}
		return EnterTextDescription;
	}

	// Callback to set the description of the launch profile.
	void OnDescriptionTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		ILauncherProfilePtr LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid())
		{
			if (NewText.EqualTo(EnterTextDescription))
			{
				LaunchProfile->SetDescription("");
			}
			else
			{
				LaunchProfile->SetDescription(NewText.ToString());
			}
		}
	}

private:

	/** Holds a pointer to the data model. */
	TSharedPtr<FProjectLauncherModel> Model;

	// Attribute for the launch profile this widget edits. 
	TAttribute<ILauncherProfilePtr> LaunchProfileAttr;

	// Cache the no description enter suggestion text
	FText EnterTextDescription;

	// Whether we show an add description blurb when the profile has no description. 
	bool bShowAddDescriptionText;

	TSharedPtr<SInlineEditableTextBlock> NameEditableTextBlock;
};

