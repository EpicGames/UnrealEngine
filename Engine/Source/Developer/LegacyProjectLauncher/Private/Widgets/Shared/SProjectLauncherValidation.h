// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfile.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"


#define LOCTEXT_NAMESPACE "SProjectLauncherValidation"


/**
 * Implements the launcher's profile validation panel.
 */
class SProjectLauncherValidation
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherValidation) { }
		SLATE_ATTRIBUTE(TSharedPtr<ILauncherProfile>, LaunchProfile)
	SLATE_END_ARGS()


public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct( const FArguments& InArgs)
	{
		LaunchProfileAttr = InArgs._LaunchProfile;

		TSharedPtr<SVerticalBox> VertBox;
		SAssignNew(VertBox, SVerticalBox);

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::CustomRolesNotSupportedYet)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::DeployedDeviceGroupRequired)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::InitialCultureNotAvailable)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::InitialMapNotAvailable)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::MalformedLaunchCommandLine)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoBuildConfigurationSelected)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoCookedCulturesSelected)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoLaunchRoleDeviceAssigned)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoPlatformSelected)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoProjectSelected)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoPackageDirectorySpecified)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::LaunchDeviceIsUnauthorized)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeCallbackMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoPlatformSDKInstalled)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::UnversionedAndIncremental)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingPatchesCanOnlyRunFromByTheBookCookMode)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingMultiLevelPatchesRequiresGeneratePatch)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::StagingBaseReleasePaksWithoutABaseReleaseVersion)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingChunksRequiresCookByTheBook)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingChunksRequiresUnrealPak)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresGeneratingChunks)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresValidDirectoryAndName)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::NoArchiveDirectorySpecified)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::IoStoreRequiresPakFiles)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::BuildTargetIsRequired)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::FallbackBuildTargetIsRequired)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch)
			];

		VertBox->AddSlot().AutoHeight()
			[
				MakeValidationMessage(TEXT("Icons.Error"), ELauncherProfileValidationErrors::AutomatedTestRequiredBuildCookRun)
			];

		static_assert(ELauncherProfileValidationErrors::Count == 31, "SProjectLauncherValidation::Construct() needs to be updated to account for modified enum values.");
		check(VertBox->NumSlots() == ELauncherProfileValidationErrors::Count);

		ChildSlot[VertBox.ToSharedRef()];
	}

protected:

	/**
	 * Creates a widget for a validation message.
	 *
	 * @param IconName The name of the message icon.
	 * @param MessageText The message text.
	 * @param MessageType The message type.
	 */
	TSharedRef<SWidget> MakeValidationMessage( const TCHAR* IconName, ELauncherProfileValidationErrors::Type Message )
	{
		return SNew(SHorizontalBox)
			.Visibility(this, &SProjectLauncherValidation::HandleValidationMessageVisibility, Message)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush(IconName))
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::FromString(LexToStringLocalized(Message)))
			];
	}

	/**
	 * Creates a widget for a validation message.
	 *
	 * @param IconName The name of the message icon.
	 * @param MessageType The message type.
	 */
	TSharedRef<SWidget> MakeCallbackMessage( const TCHAR* IconName,ELauncherProfileValidationErrors::Type Message )
	{
		return SNew(SHorizontalBox)
			.Visibility(this, &SProjectLauncherValidation::HandleValidationMessageVisibility, Message)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush(IconName))
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SProjectLauncherValidation::HandleValidationMessage, Message)
			];
	}

private:

	// Callback for getting the visibility state of a validation message.
	EVisibility HandleValidationMessageVisibility( ELauncherProfileValidationErrors::Type Error ) const
	{
		TSharedPtr<ILauncherProfile> LaunchProfile = LaunchProfileAttr.Get();
		if (!LaunchProfile.IsValid() || LaunchProfile->HasValidationError(Error))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	}

	FText HandleValidationMessage( ELauncherProfileValidationErrors::Type Error ) const
	{
		TSharedPtr<ILauncherProfile> LaunchProfile = LaunchProfileAttr.Get();
		if (LaunchProfile.IsValid())
		{
			if (LaunchProfile->HasValidationError(Error))
			{
				return FText::Format(LOCTEXT("NoPlatformSDKInstalledFmt", "A required platform SDK is missing: {0}"), FText::FromString(LaunchProfile->GetInvalidPlatform()));
			}

			return FText::GetEmpty();
		}
		return LOCTEXT("InvalidLaunchProfile", "Invalid Launch Profile.");
	}

private:

	// Attribute for the launch profile this widget shows validation for. 
	TAttribute<TSharedPtr<ILauncherProfile>> LaunchProfileAttr;
};


#undef LOCTEXT_NAMESPACE
