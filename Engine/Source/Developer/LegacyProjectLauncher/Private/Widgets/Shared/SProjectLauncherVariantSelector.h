// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "ITargetDeviceProxy.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherVariantSelector"


/**
 * Delegate type for build configuration selections.
 *
 * The first parameter is the selected build configuration.
 */
DECLARE_DELEGATE_OneParam(FOnSProjectLauncherVariantSelected, FName)


/**
 * Implements a build configuration selector widget.
 */
class SProjectLauncherVariantSelector
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SProjectLauncherVariantSelector) { }
	SLATE_EVENT(FOnSProjectLauncherVariantSelected, OnVariantSelected)
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InModel The data model.
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<ITargetDeviceProxy> DeviceProxy)
	{
		OnVariantSelected = InArgs._OnVariantSelected;

		// create instance types menu
		FMenuBuilder MenuBuilder(true, NULL);
		
		TArray<FName> Variants;
		DeviceProxy->GetVariants(Variants);

		for (TArray<FName>::TIterator It(Variants); It; ++It)
		{
			FName Variant = (*It);
			FUIAction Action(FExecuteAction::CreateSP(this, &SProjectLauncherVariantSelector::HandleMenuEntryClicked, Variant));
			MenuBuilder.AddMenuEntry(FText::FromString(Variant.ToString()), FText::GetEmpty(), FSlateIcon(), Action);
		}

		FName DefaultVariant = NAME_None;
		FUIAction Action(FExecuteAction::CreateSP(this, &SProjectLauncherVariantSelector::HandleMenuEntryClicked, DefaultVariant));
		MenuBuilder.AddMenuEntry(LOCTEXT("DefaultVariant", "Default"), FText::GetEmpty(), FSlateIcon(), Action);

		ChildSlot
		[
			// build configuration menu
			SNew(SComboButton)
			.VAlign(VAlign_Center)
			.ComboButtonStyle(FAppStyle::Get(), "ComboButton")
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("SmallFont")))
				.Text(InArgs._Text)
			]
			.ContentPadding(FMargin(6.0f, 0.0f))
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
		];
	}

private:

	// Callback for clicking a menu entry.
	void HandleMenuEntryClicked(FName Variant)
	{
		OnVariantSelected.ExecuteIfBound(Variant);
	}

private:

	// Holds a delegate to be invoked when a build configuration has been selected.
	FOnSProjectLauncherVariantSelected OnVariantSelected;
};


#undef LOCTEXT_NAMESPACE
