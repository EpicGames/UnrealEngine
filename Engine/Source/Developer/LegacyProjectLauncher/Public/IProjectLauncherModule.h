// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


/**
 * Interface for launcher UI modules.
 */
class ILegacyProjectLauncherModule
	: public IModuleInterface
{
public:

	/** Virtual destructor. */
	virtual ~ILegacyProjectLauncherModule() = default;
};
